{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "AnimGen",
	"Description": "",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimGen",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AnimGenEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PythonFoundationPackages",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "AnimationWarping",
			"Enabled": true
		},
		{
			"Name": "LearningCore",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeBasicCpu",
			"Enabled": true
		},
		{
			"Name": "AnimDatabase",
			"Enabled": true
		},
		{
			"Name": "TensorBoard",
			"Enabled": true
		},
		{
			"Name": "DrawDebugLibrary",
			"Enabled": true
		},
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFPoseSearch",
			"Enabled": true
		}
	]
}