// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGen.h"

#include "AnimGenController.h"
#include "AnimGenControl.h"

void FAnimGenModule::StartupModule() {}
	
IMPLEMENT_MODULE(FAnimGenModule, AnimGen)