// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenAutoEncoder.h"

#include "AnimDatabaseMath.h"
#include "AnimDatabasePose.h"
#include "DrawDebugLibrary.h"

#include "LearningNeuralNetwork.h"
#include "LearningFrameRangeSet.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "UObject/Package.h"
#include "Misc/ScopedSlowTask.h"

#include "LearningNeuralNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGenAutoEncoder)

#define LOCTEXT_NAMESPACE "AnimDatabase"

#if WITH_EDITOR

void UAnimGenAutoEncoderDebugDraw::InitializeDrawDebug_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& InFrameRanges, const UAnimGenAutoEncoder* InAutoEncoder) {}

void UAnimGenAutoEncoderDebugDraw::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InOriginalPoseState,
	const FAnimDatabasePoseState& InReconstructedPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor,
	const FLinearColor& OriginalColor,
	const FLinearColor& ReconstructedColor) {}

#endif

USkeleton* UAnimGenAutoEncoderTrainingSettings::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
#if WITH_EDITOR
	return !Database ? nullptr : Database->GetSkeleton();
#else
	return nullptr;
#endif
}

#if WITH_EDITOR

void UAnimGenAutoEncoderTrainingSettings::ForceRefresh()
{
	bForceRefresh = true;
}

void UAnimGenAutoEncoderTrainingSettings::LoadDatabaseAsync()
{
	Database.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda([this](const FSoftObjectPath&, UObject*) { ForceRefresh(); }));
}

void UAnimGenAutoEncoderTrainingSettings::PostEditChangeProperty(struct FPropertyChangedEvent& Event)
{
	if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, Database))
	{
		ForceRefresh();
		LoadDatabaseAsync();
	}
}

void UAnimGenAutoEncoderTrainingSettings::UpdateQueryEntries()
{
	if (const UAnimDatabase* DatabaseObject = Database.Get())
	{
		check(QueryRanges.IsValid());
		QueryEntries.SetNum(QueryRanges.FrameRangeSet->GetTotalRangeNum());
		int32 QueryEntryIdx = 0;

		const int32 EntryNum = QueryRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const int32 EntrySequenceIdx = QueryRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const UAnimSequence* AnimSequence = DatabaseObject->GetAnimSequence(EntrySequenceIdx);
			const bool bIsMirrored = DatabaseObject->GetIsMirrored(EntrySequenceIdx);

			const int32 RangeNum = QueryRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

			for (int32 EntryRangeIdx = 0; EntryRangeIdx < RangeNum; EntryRangeIdx++)
			{
				TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>& QueryEntry = QueryEntries[QueryEntryIdx];
				if (!QueryEntry.IsValid()) { QueryEntry = MakeShared<UE::AnimDatabase::Editor::FQueryEntry>(); }

				const int32 NewStartFrame = QueryRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, EntryRangeIdx);
				const int32 NewStopFrame = QueryRanges.FrameRangeSet->GetEntryRangeStop(EntryIdx, EntryRangeIdx) - 1;

				// If the entry has changed then reset selection

				if (QueryEntry->Sequence != AnimSequence ||
					QueryEntry->StartFrame != NewStartFrame ||
					QueryEntry->StopFrame != NewStopFrame ||
					QueryEntry->bIsMirrored != bIsMirrored)
				{
					QueryEntry->Sequence = AnimSequence;
					QueryEntry->StartFrame = NewStartFrame;
					QueryEntry->StopFrame = NewStopFrame;
					QueryEntry->bIsMirrored = bIsMirrored;
					QueryEntry->bIsSelected = false;
				}

				// Here we use a hash of the sequence, start, and stop frames to produce the color

				const uint32 HashData[5] =
				{
					(uint32)RangeIndentifierColorSeed,
					GetTypeHash(AnimSequence),
					(uint32)QueryEntry->StartFrame,
					(uint32)QueryEntry->StopFrame,
					(uint32)QueryEntry->bIsMirrored
				};

				const uint32 ColorHash = CityHash32((const char*)HashData, sizeof(HashData));
				const uint8 Hue = ColorHash & 0x000000FF;
				const uint8 Saturation = ((ColorHash & 0x0000FF00) >> 8) % 25 + 230;
				const uint8 Brightness = ((ColorHash & 0x00FF0000) >> 16) % 50 + 205;

				QueryEntry->Color = FLinearColor::MakeFromHSV8(Hue, Saturation, Brightness);

				QueryEntryIdx++;
			}
		}

		check(QueryEntryIdx == QueryEntries.Num());
	}
	else
	{
		QueryEntries.Reset();
	}
}

bool UAnimGenAutoEncoderTrainingSettings::Update()
{
	if (const UAnimDatabase* DatabaseObject = Database.Get())
	{
		// Otherwise check for changes to various things to detect update

		bool bQueryUpdateRequired = bForceRefresh;
		bForceRefresh = false;

		// Check if Database has been updated

		const uint32 CurrentContentHash = DatabaseObject->GetContentHash();
		if (CurrentContentHash != DatabaseContentHash)
		{
			DatabaseContentHash = CurrentContentHash;
			bQueryUpdateRequired = true;
		}

		// Check if ActiveFrameRanges is out-of-date

		if (bQueryUpdateRequired)
		{
			if (FrameRanges)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingTrainingRanges", "Rebuilding Training Frame Ranges"));
				SlowTask.MakeDialog();

				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(DatabaseObject, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(DatabaseObject), FrameRanges);
				if (!QueryRanges.IsValid()) { QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges(); }
			}
			else
			{
				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(DatabaseObject);
			}
			check(QueryRanges.IsValid());

			UpdateQueryEntries();
		}

		check(QueryRanges.IsValid());

		// Update Debug Drawer

		if (bQueryUpdateRequired && (DatabaseDebugDrawer || AutoEncoderDebugDrawer))
		{
			FScopedSlowTask SlowTask(0.0f, LOCTEXT("ReinitializingDebugDrawData", "Reinitializing Debug Draw"));
			SlowTask.MakeDialog();

			if (DatabaseDebugDrawer)
			{
				DatabaseDebugDrawer->InitializeDrawDebug(DatabaseObject, QueryRanges);
			}

			if (AutoEncoderDebugDrawer)
			{
				AutoEncoderDebugDrawer->InitializeDrawDebug(DatabaseObject, QueryRanges, Cast<UAnimGenAutoEncoder>(GetOuter()));
			}
		}

		// Update Selection

		SelectedRanges.Reset();

		const int32 QueryRangeNum = QueryEntries.Num();

		for (int32 QueryRangeIdx = 0; QueryRangeIdx < QueryRangeNum; QueryRangeIdx++)
		{
			if (QueryEntries[QueryRangeIdx]->bIsSelected)
			{
				SelectedRanges.Add(QueryRangeIdx);
			}
		}

		// Return if query update required

		return bQueryUpdateRequired;
	}

	if (Database.IsNull() && (DatabaseContentHash != 0 || bForceRefresh))
	{
		bForceRefresh = false;
		DatabaseContentHash = 0;
		QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
		UpdateQueryEntries();
		SelectedRanges.Reset();
		return true;
	}

	return false;
}

#endif

UAnimGenAutoEncoder::UAnimGenAutoEncoder(const FObjectInitializer& ObjectInitializer)
{
#if WITH_EDITOR
	ViewportSettings = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimGenAutoEncoderViewportSettings>(this, TEXT("ViewportSettings"));
	TrainingSettings = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimGenAutoEncoderTrainingSettings>(this, TEXT("TrainingSettings"));
#endif
}

bool UAnimGenAutoEncoder::IsValid() const
{
	return 
		!BoneNames.IsEmpty() && 
		EncoderNetwork && EncoderNetwork->GetUpdateNumber() > 0 &&
		DecoderNetwork && DecoderNetwork->GetUpdateNumber() > 0;
}

void UAnimGenAutoEncoder::Invalidate()
{
	EncoderNetwork = nullptr;
	DecoderNetwork = nullptr;
	BoneNames.Empty();
	BoneParents.Empty();
	DefaultBoneLocations.Empty();
	DefaultBoneRotations.Empty();
	DefaultBoneScales.Empty();
	AutoEncodedBoneLocationIndices.Empty();
	AutoEncodedBoneRotationIndices.Empty();
	AutoEncodedBoneScaleIndices.Empty();
	AttributeNames.Empty();
	AttributeTypes.Empty();
}

int32 UAnimGenAutoEncoder::GetEncodingSize() const
{
	return EncoderNetwork ? EncoderNetwork->GetOutputSize() : 0;
}

int32 UAnimGenAutoEncoder::GetPoseVectorSize() const
{
	return UE::AnimDatabase::PoseData::PoseVectorSize(
		AutoEncodedBoneLocationIndices.Num(),
		AutoEncodedBoneRotationIndices.Num(),
		AutoEncodedBoneScaleIndices.Num(),
		AttributeTypes);
}

int32 UAnimGenAutoEncoder::GetBoneNum() const
{
	return BoneNames.Num();
}

FName UAnimGenAutoEncoder::GetBoneName(const int32 BoneIdx) const
{
	return BoneNames.IsValidIndex(BoneIdx) ? BoneNames[BoneIdx] : NAME_None;
}

const TArray<FName>& UAnimGenAutoEncoder::GetBoneNames() const
{
	return BoneNames;
}

int32 UAnimGenAutoEncoder::FindBoneIndex(const FName BoneName) const
{
	return BoneNames.Find(BoneName);
}

void UAnimGenAutoEncoder::FindBoneIndices(TArray<int32>& OutBoneIndices, const TArray<FName>& InBoneNames) const
{
	OutBoneIndices.SetNumUninitialized(InBoneNames.Num());
	FindBoneIndicesFromArrayViews(OutBoneIndices, InBoneNames);
}

void UAnimGenAutoEncoder::FindBoneIndicesFromArrayViews(TArrayView<int32> OutBoneIndices, const TArrayView<const FName> InBoneNames) const
{
	check(OutBoneIndices.Num() == InBoneNames.Num());
	const int32 BoneNum = InBoneNames.Num();
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutBoneIndices[BoneIdx] = FindBoneIndex(InBoneNames[BoneIdx]);
	}
}

int32 UAnimGenAutoEncoder::GetAttributeNum() const
{
	return AttributeNames.Num();
}

FName UAnimGenAutoEncoder::GetAttributeName(const int32 FrameAttributeIdx) const
{
	if (FrameAttributeIdx < 0 || FrameAttributeIdx >= AttributeNames.Num()) { return NAME_None; }
	return AttributeNames[FrameAttributeIdx];
}

EAnimDatabaseAttributeType UAnimGenAutoEncoder::GetAttributeType(const int32 FrameAttributeIdx) const
{
	if (FrameAttributeIdx < 0 || FrameAttributeIdx >= AttributeTypes.Num()) { return EAnimDatabaseAttributeType::Null; }
	return AttributeTypes[FrameAttributeIdx];
}

int32 UAnimGenAutoEncoder::FindAttributeIndex(const FName FrameAttributeName) const
{
	return AttributeNames.Find(FrameAttributeName);
}

const TArray<FName>& UAnimGenAutoEncoder::GetAttributeNames() const
{
	return AttributeNames;
}

const TArray<EAnimDatabaseAttributeType>& UAnimGenAutoEncoder::GetAttributeTypes() const
{
	return AttributeTypes;
}

int32 UAnimGenAutoEncoder::GetContentHash() const
{
	if (EncoderNetwork && DecoderNetwork)
	{
		return EncoderNetwork->GetContentHash() ^ DecoderNetwork->GetContentHash();
	}
	else
	{
		return 0;
	}
}

const TArray<int32>& UAnimGenAutoEncoder::GetBoneParents() const
{
	return BoneParents;
}

int32 UAnimGenAutoEncoder::GetBoneParent(const int32 BoneIdx) const
{
	return BoneParents.IsValidIndex(BoneIdx) ? BoneParents[BoneIdx] : INDEX_NONE;
}

void UAnimGenAutoEncoder::ToPoseVectors(
	const TLearningArrayView<2, float> OutPoseVectors,
	const UE::AnimDatabase::FPoseDataConstView& InPoseData) const
{
	ToPoseVectors(OutPoseVectors, InPoseData, UE::Learning::FIndexSet(0, GetBoneNum()));
}

void UAnimGenAutoEncoder::FromPoseVectors(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const TLearningArrayView<2, const float> InPoseVectors,
	const TLearningArrayView<1, const FVector> InRootLocations,
	const TLearningArrayView<1, const FQuat4f> InRootRotations) const
{
	FromPoseVectors(OutPoseData, InPoseVectors, InRootLocations, InRootRotations, UE::Learning::FIndexSet(0, GetBoneNum()));
}

void UAnimGenAutoEncoder::SetDefaultPoseData(const UE::AnimDatabase::FPoseDataView& OutPoseData) const
{
	SetDefaultPoseData(OutPoseData, UE::Learning::FIndexSet(0, GetBoneNum()));
}

void UAnimGenAutoEncoder::ToPoseVectors(
	const TLearningArrayView<2, float> OutPoseVectors,
	const UE::AnimDatabase::FPoseDataConstView& InPoseData,
	const UE::Learning::FIndexSet BoneIndices) const
{
	UE::AnimDatabase::PoseData::ToPoseVectors(
		OutPoseVectors,
		InPoseData,
		AutoEncodedBoneLocationIndices,
		AutoEncodedBoneRotationIndices,
		AutoEncodedBoneScaleIndices,
		DefaultBoneLocations,
		DefaultBoneRotations,
		DefaultBoneScales,
		BoneIndices);
}

void UAnimGenAutoEncoder::FromPoseVectors(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const TLearningArrayView<2, const float> InPoseVectors,
	const TLearningArrayView<1, const FVector> InRootLocations,
	const TLearningArrayView<1, const FQuat4f> InRootRotations,
	const UE::Learning::FIndexSet BoneIndices) const
{
	UE::AnimDatabase::PoseData::FromPoseVectors(
		OutPoseData,
		InPoseVectors,
		InRootLocations,
		InRootRotations,
		AutoEncodedBoneLocationIndices,
		AutoEncodedBoneRotationIndices,
		AutoEncodedBoneScaleIndices,
		DefaultBoneLocations,
		DefaultBoneRotations,
		DefaultBoneScales,
		BoneIndices);
}

void UAnimGenAutoEncoder::SetDefaultPoseData(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const UE::Learning::FIndexSet BoneIndices) const
{
	UE::AnimDatabase::PoseData::SetDefaultPoseData(
		OutPoseData,
		DefaultBoneLocations,
		DefaultBoneRotations,
		DefaultBoneScales,
		BoneIndices);
}

void UAnimGenAutoEncoder::NormalizePoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const
{
	check(InOutPoseVectors.Num<1>() == PoseVectorOffset.Num());
	check(InOutPoseVectors.Num<1>() == PoseVectorScale.Num());

	UE::AnimDatabase::Math::NormalizeInplace(InOutPoseVectors, PoseVectorOffset, PoseVectorScale);
}

void UAnimGenAutoEncoder::DenormalizePoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const
{
	check(InOutPoseVectors.Num<1>() == PoseVectorOffset.Num());
	check(InOutPoseVectors.Num<1>() == PoseVectorScale.Num());

	UE::AnimDatabase::Math::DenormalizeInplace(InOutPoseVectors, PoseVectorOffset, PoseVectorScale);
}

void UAnimGenAutoEncoder::ClampPoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const
{
	check(InOutPoseVectors.Num<1>() == PoseVectorMin.Num());
	check(InOutPoseVectors.Num<1>() == PoseVectorMax.Num());

	UE::AnimDatabase::Math::ClampInplace(InOutPoseVectors, PoseVectorMin, PoseVectorMax);
}

#if WITH_EDITOR

void UAnimGenAutoEncoderDebugDraw_ContactTimings::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InOriginalPoseState,
	const FAnimDatabasePoseState& InReconstructedPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor,
	const FLinearColor& OriginalColor,
	const FLinearColor& ReconstructedColor)
{
	const int32 LeftToeBoneIndex = InAutoEncoder->FindBoneIndex(LeftToeBoneName);
	const int32 LeftToeContactAttributeIndex = InAutoEncoder->FindAttributeIndex(LeftContactCurveAttribute);

	if (LeftToeBoneIndex != INDEX_NONE && LeftToeContactAttributeIndex != INDEX_NONE)
	{
		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InOriginalPoseState, LeftToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(OriginalColor),
			true,
			ContactCircleRadius * UAnimDatabasePoseStateLibrary::PoseStateFloatAttribute(InOriginalPoseState, LeftToeContactAttributeIndex));

		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InReconstructedPoseState, LeftToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(ReconstructedColor),
			true,
			ContactCircleRadius * UAnimDatabasePoseStateLibrary::PoseStateFloatAttribute(InReconstructedPoseState, LeftToeContactAttributeIndex));
	}

	const int32 RightToeBoneIndex = InAutoEncoder->FindBoneIndex(RightToeBoneName);
	const int32 RightToeContactAttributeIndex = InAutoEncoder->FindAttributeIndex(RightContactCurveAttribute);

	if (RightToeBoneIndex != INDEX_NONE && RightToeContactAttributeIndex != INDEX_NONE)
	{
		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InOriginalPoseState, RightToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(OriginalColor),
			true,
			ContactCircleRadius * UAnimDatabasePoseStateLibrary::PoseStateFloatAttribute(InOriginalPoseState, RightToeContactAttributeIndex));

		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InReconstructedPoseState, RightToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(ReconstructedColor),
			true,
			ContactCircleRadius * UAnimDatabasePoseStateLibrary::PoseStateFloatAttribute(InReconstructedPoseState, RightToeContactAttributeIndex));
	}
}

void UAnimGenAutoEncoderDebugDraw_Attributes::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InOriginalPoseState,
	const FAnimDatabasePoseState& InReconstructedPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor,
	const FLinearColor& OriginalColor,
	const FLinearColor& ReconstructedColor)
{
	const int32 AttributeNum = UAnimDatabasePoseStateLibrary::PoseStateAttributeNum(InOriginalPoseState);

	const float Offset = VerticalSpacing + CharacterIdx * 2 * AttributeNum * VerticalSpacing;

	FDrawDebugCanvasTextSettings Settings;
	Settings.bShadow = true;

	for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
	{
		UDrawDebugLibrary::DrawDebugCanvasStringView(
			CanvasDrawer,
			FVector2D(VerticalSpacing, Offset + AttributeIdx * 2 * VerticalSpacing),
			FString::Printf(TEXT("%s (%s)"),
				*UAnimDatabasePoseStateLibrary::PoseStateAttributeName(InOriginalPoseState, AttributeIdx).ToString(),
				UAnimDatabaseFrameAttributeLibrary::AttributeTypeNameInternal(UAnimDatabasePoseStateLibrary::PoseStateAttributeType(InOriginalPoseState, AttributeIdx))),
			IdentifierColor,
			Settings);
	}

	for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
	{
		UDrawDebugLibrary::DrawDebugCanvasStringView(
			CanvasDrawer,
			FVector2D(VerticalSpacing + PropertyHorizontalOffset, Offset + AttributeIdx * 2 * VerticalSpacing),
			UAnimDatabasePoseStateLibrary::PoseStateIsAttributeActive(InOriginalPoseState, AttributeIdx) ?
			*UAnimDatabasePoseStateLibrary::PoseStateAttributeString(InOriginalPoseState, AttributeIdx) : TEXT("Inactive"),
			OriginalColor,
			Settings);
	}

	for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
	{
		UDrawDebugLibrary::DrawDebugCanvasStringView(
			CanvasDrawer,
			FVector2D(VerticalSpacing + PropertyHorizontalOffset, Offset + AttributeIdx * 2 * VerticalSpacing + VerticalSpacing),
			UAnimDatabasePoseStateLibrary::PoseStateIsAttributeActive(InReconstructedPoseState, AttributeIdx) ?
			*UAnimDatabasePoseStateLibrary::PoseStateAttributeString(InReconstructedPoseState, AttributeIdx) : TEXT("Inactive"),
			ReconstructedColor,
			Settings);
	}
}

#endif

#undef LOCTEXT_NAMESPACE
