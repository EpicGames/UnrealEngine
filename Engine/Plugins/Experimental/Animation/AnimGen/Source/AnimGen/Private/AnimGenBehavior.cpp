// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenBehavior.h"
#include "AnimDatabasePose.h"
#include "AnimDatabaseMath.h"

#include "LearningRandom.h"

#include "AnimGenLog.h"
#include "InputCoreTypes.h"
#include "Math/SpringMath.h"
#include "DrawDebugLibrary.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGenBehavior)

#if WITH_EDITOR

FAnimGenControlSchemaElement UAnimGenBehavior::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	UE_LOGFMT(LogAnimGen, Error, "SpecifyControl must be implemented by Behavior.");
	return FAnimGenControlSchemaElement();
}

void UAnimGenBehavior::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const {}

void UAnimGenBehavior::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	UE_LOGFMT(LogAnimGen, Error, "MakeControlSets must be implemented by Behavior.");
	OutControlSets.Empty();
}

FAnimGenControlSchemaElement UAnimGenBehavior_Uncontrolled::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyNullControl(InControlSchema);
}

void UAnimGenBehavior_Uncontrolled::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	OutControlSets.Add(UAnimGenControls::MakeNullControlSet(InControlObject, InFrameRanges));
}

FAnimGenControlSchemaElement UAnimGenBehavior_TrajectoryFollow::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyTrajectoryControl(InControlSchema, TrajectorySampleNum);
}

void UAnimGenBehavior_TrajectoryFollow::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	// Trim TrajectoryFutureTime and TrajectoryPastTime from all anim sequences since we don't have those trajectories
	const FAnimDatabaseFrameRanges TrimmedRanges = UAnimDatabaseFrameRangesLibrary::FrameRangesIntersection(
		InFrameRanges, 
		UAnimDatabaseFrameRangesLibrary::FrameRangesTrim(
			UAnimDatabaseFrameRangesLibrary::FrameRangesAll(InDatabase),
			FMath::RoundToInt(TrajectoryPastTime * InDatabase->GetFrameRate().AsDecimal()),
			FMath::RoundToInt(TrajectoryFutureTime * InDatabase->GetFrameRate().AsDecimal())));

	OutControlSets.Add(UAnimGenControls::MakeTrajectoryControlSetFromDatabase(InControlObject, InDatabase, TrimmedRanges, TrajectorySampleNum, TrajectoryFutureTime, TrajectoryPastTime, TrajectoryForwardVector));
}

void UAnimGenBehavior_TrajectoryFollow::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	const FTransform RootTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState);
	UAnimGenControls::DrawDebugTrajectoryControl(Drawer, InControlObject, InControlObjectElement, RootTransform, IdentifierColor, 25.0f, 5.0f, 1.0f, 1.0f);
}


FAnimGenControlSchemaElement UAnimGenBehavior_VelocityFollow::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyStructControlFromArrayViews(InControlSchema,
		{
			TEXT("LinearVelocity"),
			TEXT("AngularVelocity"),
		},
		{
			UAnimGenControls::SpecifyLinearVelocityControl(InControlSchema),
			UAnimGenControls::SpecifyAngularVelocityControl(InControlSchema),
		});
}

void UAnimGenBehavior_VelocityFollow::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	OutControlSets.Add(UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject,
		{
			TEXT("LinearVelocity"),
			TEXT("AngularVelocity"),
		},
		{
			UAnimGenControls::MakeRootLinearVelocityControlSetFromDatabase(InControlObject, InDatabase, InFrameRanges),
			UAnimGenControls::MakeRootAngularVelocityControlSetFromDatabase(InControlObject, InDatabase, InFrameRanges),
		}));
}

void UAnimGenBehavior_VelocityFollow::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	FAnimGenControlObjectElement LinearVelocityControl, AngularVelocityControl;
	UAnimGenControls::GetStructControlElement(LinearVelocityControl, InControlObject, InControlObjectElement, TEXT("LinearVelocity"));
	UAnimGenControls::GetStructControlElement(AngularVelocityControl, InControlObject, InControlObjectElement, TEXT("AngularVelocity"));

	const FTransform RootTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState);

	UAnimGenControls::DrawDebugLinearVelocityControl(Drawer, InControlObject, LinearVelocityControl, RootTransform, RootTransform.GetLocation() + FVector(0.0f, 0.0f, 10.0f), IdentifierColor, 1.0f, 1.0f);
	UAnimGenControls::DrawDebugAngularVelocityControl(Drawer, InControlObject, AngularVelocityControl, RootTransform, RootTransform.GetLocation() + FVector(0.0f, 0.0f, 10.0f), IdentifierColor, 1.0f, 1.0f);
}

FAnimGenControlSchemaElement UAnimGenBehavior_MoveToTarget::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyStructControlFromArrayViews(InControlSchema,
		{
			TEXT("Location"),
			TEXT("Direction"),
		},
		{
			UAnimGenControls::SpecifyLocationControl(InControlSchema),
			UAnimGenControls::SpecifyOptionalControl(InControlSchema, UAnimGenControls::SpecifyDirectionControl(InControlSchema)),
		});
}

void UAnimGenBehavior_MoveToTarget::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	const FAnimGenControlSet LocationControlSet = UAnimGenControls::MakeRootLocationAtRangeEndControlSetFromDatabase(InControlObject, InDatabase, InFrameRanges);
	const FAnimGenControlSet DirectionControlSet = UAnimGenControls::MakeRootDirectionAtRangeEndControlSetFromDatabase(InControlObject, InDatabase, InFrameRanges);

	OutControlSets.Add(UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject,
		{
			TEXT("Location"),
			TEXT("Direction"),
		},
		{
			LocationControlSet,
			UAnimGenControls::MakeOptionalValidControlSet(InControlObject, DirectionControlSet),
		}));

	OutControlSets.Add(UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject,
		{
			TEXT("Location"),
			TEXT("Direction"),
		},
		{
			LocationControlSet,
			UAnimGenControls::MakeOptionalNullControlSet(InControlObject, InFrameRanges),
		}));
}

void UAnimGenBehavior_MoveToTarget::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	FAnimGenControlObjectElement LocationControl, OptionalDirectionControl;
	UAnimGenControls::GetStructControlElement(LocationControl, InControlObject, InControlObjectElement, TEXT("Location"));
	UAnimGenControls::GetStructControlElement(OptionalDirectionControl, InControlObject, InControlObjectElement, TEXT("Direction"));

	const FTransform RootTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState);

	UAnimGenControls::DrawDebugLocationControl(Drawer, InControlObject, LocationControl, RootTransform, IdentifierColor, 10.0f, 1.0f);

	FAnimGenControlObjectElement DirectionControl;
	EAnimGenOptionalControl OptionalControl;
	UAnimGenControls::GetOptionalControl(OptionalControl, DirectionControl, InControlObject, OptionalDirectionControl);
	if (OptionalControl == EAnimGenOptionalControl::Valid)
	{
		FVector Location = FVector::ZeroVector;
		UAnimGenControls::GetLocationControl(Location, InControlObject, LocationControl, RootTransform);
		UAnimGenControls::DrawDebugDirectionControl(Drawer, InControlObject, DirectionControl, RootTransform, Location + FVector(0.0f, 0.0, 1.0f), IdentifierColor, 25.0f, 1.0f, 1.0f);
	}
}

FAnimGenControlSchemaElement UAnimGenBehavior_TrajectoryInteraction::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyStructControlFromArrayViews(InControlSchema,
		{
			TEXT("Trajectory"),
			TEXT("Interaction")
		},
		{
			UAnimGenControls::SpecifyTrajectoryControl(InControlSchema, TrajectorySampleNum),
			UAnimGenControls::SpecifyOptionalControl(InControlSchema, UAnimGenControls::SpecifyStructControlFromArrayViews(InControlSchema,
				{
					TEXT("Location"),
					TEXT("Direction"),
				},
				{
					UAnimGenControls::SpecifyLocationControl(InControlSchema),
					UAnimGenControls::SpecifyDirectionControl(InControlSchema)
				}))
		});
}

void UAnimGenBehavior_TrajectoryInteraction::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	// Trim TrajectoryFutureTime and TrajectoryPastTime from all anim sequences since we don't have those trajectories
	const FAnimDatabaseFrameRanges TrimmedRanges = UAnimDatabaseFrameRangesLibrary::FrameRangesIntersection(
		InFrameRanges,
		UAnimDatabaseFrameRangesLibrary::FrameRangesTrim(
			UAnimDatabaseFrameRangesLibrary::FrameRangesAll(InDatabase),
			FMath::RoundToInt(TrajectoryPastTime * InDatabase->GetFrameRate().AsDecimal()),
			FMath::RoundToInt(TrajectoryFutureTime * InDatabase->GetFrameRate().AsDecimal())));

	FAnimGenControlSet InteractionFacingControlSet;

	if (bUseMirroredInteractionForwardVector)
	{
		const int32 InteractionBoneIndex = InDatabase->FindBoneIndex(InteractionBoneName);

		const FAnimDatabaseFrameAttribute GlobalDirectionAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSelect(
			UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromNotMirrored(InDatabase, TrimmedRanges),
			UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttribute(InDatabase, TrimmedRanges, InteractionBoneIndex, InteractionForwardVector),
			UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttribute(InDatabase, TrimmedRanges, InteractionBoneIndex, InteractionMirroredForwardVector));

		InteractionFacingControlSet = UAnimGenControls::MakeDirectionControlSet(InControlObject, GlobalDirectionAttribute, UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(InDatabase, TrimmedRanges));
	}
	else
	{
		InteractionFacingControlSet = UAnimGenControls::MakeBoneGlobalDirectionControlSetFromName(InControlObject, InDatabase, TrimmedRanges, InteractionBoneName, InteractionForwardVector);
	}

	OutControlSets.Add(UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject,
		{
			TEXT("Trajectory"),
			TEXT("Interaction")
		},
		{
			UAnimGenControls::MakeTrajectoryControlSetFromDatabase(InControlObject, InDatabase, TrimmedRanges, TrajectorySampleNum, TrajectoryFutureTime, TrajectoryPastTime, TrajectoryForwardVector),
			UAnimGenControls::MakeOptionalControlSet(InControlObject, UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject, 
					{
						TEXT("Location"),
						TEXT("Direction")
					}, 
					{
						UAnimGenControls::MakeBoneGlobalLocationControlSetFromName(InControlObject, InDatabase, TrimmedRanges, InteractionBoneName),
						InteractionFacingControlSet,
					}), 
					UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(
						InteractionRanges ? UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, TrimmedRanges, InteractionRanges) : TrimmedRanges, 
						TrimmedRanges))
		}));
}

void UAnimGenBehavior_TrajectoryInteraction::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	FAnimGenControlObjectElement TrajectoryControl, InteractionControl;
	UAnimGenControls::GetStructControlElement(TrajectoryControl, InControlObject, InControlObjectElement, TEXT("Trajectory"));
	UAnimGenControls::GetStructControlElement(InteractionControl, InControlObject, InControlObjectElement, TEXT("Interaction"));

	const FTransform RootTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState);

	UAnimGenControls::DrawDebugTrajectoryControl(Drawer, InControlObject, TrajectoryControl, RootTransform, IdentifierColor, 25.0f, 5.0f, 1.0f, 1.0f);

	EAnimGenOptionalControl Option;
	FAnimGenControlObjectElement InteractionControlStruct;
	UAnimGenControls::GetOptionalControl(Option, InteractionControlStruct, InControlObject, InteractionControl);

	if (Option == EAnimGenOptionalControl::Valid)
	{
		FAnimGenControlObjectElement LocationControl, DirectionControl;
		UAnimGenControls::GetStructControlElement(LocationControl, InControlObject, InteractionControlStruct, TEXT("Location"));
		UAnimGenControls::GetStructControlElement(DirectionControl, InControlObject, InteractionControlStruct, TEXT("Direction"));

		FVector InteractionLocation = FVector::ZeroVector;
		UAnimGenControls::GetLocationControl(InteractionLocation, InControlObject, LocationControl, RootTransform);

		UAnimGenControls::DrawDebugLocationControl(Drawer, InControlObject, LocationControl, RootTransform, IdentifierColor, 10.0f, 2.0f);
		UAnimGenControls::DrawDebugDirectionControl(Drawer, InControlObject, DirectionControl, RootTransform, InteractionLocation, IdentifierColor, 25.0f, 1.0f, 2.0f);
	}
}


FAnimGenControlSchemaElement UAnimGenBehavior_Tagged::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	const int32 TagNum = TagRanges.Num();

	TArray<FName, TInlineAllocator<32>> TagNames;
	TagNames.Reserve(TagNum);

	for (int32 TagIdx = 0; TagIdx < TagNum; TagIdx++)
	{
		TagNames.AddUnique(TagRanges[TagIdx].Name);
	}

	return UAnimGenControls::SpecifyStructControlFromArrayViews(InControlSchema,
		{
			TEXT("Behavior"),
			TEXT("Tags"),
		},
		{
			Behavior ? Behavior->SpecifyControl(InControlSchema) : UAnimGenControls::SpecifyNullControl(InControlSchema),
			UAnimGenControls::SpecifyNamedInclusiveDiscreteControlFromArrayView(InControlSchema, TagNames),
		});
}

void UAnimGenBehavior_Tagged::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	const int32 TagNum = TagRanges.Num();

	TArray<FName, TInlineAllocator<32>> TagNames;
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<32>> TagFrameRanges;
	TagNames.Reserve(TagNum);
	TagFrameRanges.Reserve(TagNum);

	for (int32 TagIdx = 0; TagIdx < TagNum; TagIdx++)
	{
		if (TagNames.Contains(TagRanges[TagIdx].Name)) { continue; }

		const FAnimDatabaseFrameRanges Ranges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, TagRanges[TagIdx].FrameRanges);

		if (Ranges.IsValid() && !Ranges.FrameRangeSet->IsEmpty())
		{
			TagNames.Add(TagRanges[TagIdx].Name);
			TagFrameRanges.Add(Ranges);
		}
	}

	TArray<FAnimGenControlSet> SubBehaviorControlSets;
	if (Behavior)
	{
		Behavior->MakeControlSets(SubBehaviorControlSets, InControlObject, InDatabase, InFrameRanges);
	}

	for (const FAnimGenControlSet& ControlSet : SubBehaviorControlSets)
	{
		OutControlSets.Add(UAnimGenControls::MakeStructControlSetFromArrayViews(InControlObject,
			{
				TEXT("Behavior"),
				TEXT("Tags")
			},
			{
				ControlSet,
				UAnimGenControls::MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews(InControlObject, UAnimGenControls::ControlSetFrameRanges(ControlSet), TagNames, TagFrameRanges),
			}));
	}
}

void UAnimGenBehavior_Tagged::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	FAnimGenControlObjectElement BehaviorControl, TagsControl;
	UAnimGenControls::GetStructControlElement(BehaviorControl, InControlObject, InControlObjectElement, TEXT("Behavior"));
	UAnimGenControls::GetStructControlElement(TagsControl, InControlObject, InControlObjectElement, TEXT("Tags"));

	UAnimGenControls::DrawDebugNamedInclusiveDiscreteControl(
		Drawer, 
		InControlObject, 
		TagsControl, 
		TEXT(""), 
		UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState).GetLocation() + TagDebugDrawOffset,
		FRotator(0.0f, -90.0f, 0.0f), 
		10.0f, 
		IdentifierColor);

	if (Behavior)
	{
		Behavior->DrawDebugControl(
			Drawer,
			CanvasDrawer,
			InControlObject,
			BehaviorControl,
			InPoseState,
			InDatabase,
			InAutoEncoder,
			CharacterIdx,
			SequenceIdx,
			SequenceTime,
			RangeStart,
			RangeLength,
			IdentifierColor);
	}
}

FAnimGenControlSchemaElement UAnimGenBehavior_Optional::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyOptionalControl(
		InControlSchema,
		Behavior ? Behavior->SpecifyControl(InControlSchema) : UAnimGenControls::SpecifyNullControl(InControlSchema),
		EncodingSize);
}

void UAnimGenBehavior_Optional::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	TArray<FAnimGenControlSet> SubControlSets;
	if (Behavior)
	{
		Behavior->MakeControlSets(SubControlSets, InControlObject, InDatabase, InFrameRanges);
	}

	for (const FAnimGenControlSet& SubControlSet : SubControlSets)
	{
		OutControlSets.Add(UAnimGenControls::MakeOptionalValidControlSet(InControlObject, SubControlSet));
	}

	OutControlSets.Add(UAnimGenControls::MakeOptionalNullControlSet(InControlObject, InFrameRanges));
}

void UAnimGenBehavior_Optional::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	EAnimGenOptionalControl Option = EAnimGenOptionalControl::Null;
	FAnimGenControlObjectElement SubControl;
	UAnimGenControls::GetOptionalControl(Option, SubControl, InControlObject, InControlObjectElement);

	if (Behavior && Option == EAnimGenOptionalControl::Valid)
	{
		Behavior->DrawDebugControl(
			Drawer,
			CanvasDrawer,
			InControlObject,
			SubControl,
			InPoseState,
			InDatabase,
			InAutoEncoder,
			CharacterIdx,
			SequenceIdx,
			SequenceTime,
			RangeStart,
			RangeLength,
			IdentifierColor);
	}
}

FAnimGenControlSchemaElement UAnimGenBehavior_Encoded::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	return UAnimGenControls::SpecifyEncodingControl(
		InControlSchema,
		Behavior ? Behavior->SpecifyControl(InControlSchema) : UAnimGenControls::SpecifyNullControl(InControlSchema),
		EncodingSize,
		HiddenLayerNum,
		ActivationFunction);
}

void UAnimGenBehavior_Encoded::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	TArray<FAnimGenControlSet> SubControlSets;
	if (Behavior)
	{
		Behavior->MakeControlSets(SubControlSets, InControlObject, InDatabase, InFrameRanges);
	}

	for (const FAnimGenControlSet& SubControlSet : SubControlSets)
	{
		OutControlSets.Add(UAnimGenControls::MakeEncodingControlSet(InControlObject, SubControlSet));
	}
}

void UAnimGenBehavior_Encoded::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	FAnimGenControlObjectElement SubControl;
	UAnimGenControls::GetEncodingControl(SubControl, InControlObject, InControlObjectElement);

	if (Behavior)
	{
		Behavior->DrawDebugControl(
			Drawer,
			CanvasDrawer,
			InControlObject,
			SubControl,
			InPoseState,
			InDatabase,
			InAutoEncoder,
			CharacterIdx,
			SequenceIdx,
			SequenceTime,
			RangeStart,
			RangeLength,
			IdentifierColor);
	}
}

FAnimGenControlSchemaElement UAnimGenBehavior_Multi::SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const
{
	TArray<FName, TInlineAllocator<32>> SubBehaviorNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<32>> SubBehaviorElements;

	const int32 BehaviorNum = Behaviors.Num();

	for (int32 BehaviorIdx = 0 ; BehaviorIdx < BehaviorNum; BehaviorIdx++)
	{
		if (Behaviors[BehaviorIdx].Behavior)
		{
			SubBehaviorNames.Add(Behaviors[BehaviorIdx].Name);
			SubBehaviorElements.Add(UAnimGenControls::SpecifyEncodingControl(InControlSchema, Behaviors[BehaviorIdx].Behavior->SpecifyControl(InControlSchema)));
		}
	}

	return UAnimGenControls::SpecifyExclusiveUnionControlFromArrayViews(InControlSchema, SubBehaviorNames, SubBehaviorElements);
}

void UAnimGenBehavior_Multi::MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const
{
	const int32 BehaviorNum = Behaviors.Num();

	for (int32 BehaviorIdx = 0; BehaviorIdx < BehaviorNum; BehaviorIdx++)
	{
		if (Behaviors[BehaviorIdx].Behavior)
		{
			const FAnimDatabaseFrameRanges BehaviorFramesRanges = Behaviors[BehaviorIdx].FrameRanges ? 
				UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, Behaviors[BehaviorIdx].FrameRanges) : InFrameRanges;

			TArray<FAnimGenControlSet> SubBehaviorControlSets;
			Behaviors[BehaviorIdx].Behavior->MakeControlSets(SubBehaviorControlSets, InControlObject, InDatabase, BehaviorFramesRanges);

			for (const FAnimGenControlSet& ControlSet : SubBehaviorControlSets)
			{
				OutControlSets.Add(UAnimGenControls::MakeExclusiveUnionControlSet(InControlObject, Behaviors[BehaviorIdx].Name, UAnimGenControls::MakeEncodingControlSet(InControlObject, ControlSet)));
			}
		}
	}
}

void UAnimGenBehavior_Multi::DrawDebugControl_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement InControlObjectElement,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const UAnimGenAutoEncoder* InAutoEncoder,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) const
{
	const FTransform RootTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(InPoseState);

	FName BehaviorName = NAME_None;
	FAnimGenControlObjectElement EncodingElement, BehaviorElement;
	UAnimGenControls::GetExclusiveUnionControl(BehaviorName, EncodingElement, InControlObject, InControlObjectElement);
	UAnimGenControls::GetEncodingControl(BehaviorElement, InControlObject, EncodingElement);

	FDrawDebugStringSettings Settings;
	Settings.Height = 10.0f;
	Settings.bCenterHorizontally = true;
	Settings.bCenterVertically = true;

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = IdentifierColor;

	UDrawDebugLibrary::DrawDebugName(Drawer, BehaviorName, RootTransform.GetLocation() + FVector(0.0, 0.0, 240.0f), FRotator(0.0f, -90.0f, 0.0f), LineStyle, true, Settings);

	const int32 BehaviorNum = Behaviors.Num();

	for (int32 BehaviorIdx = 0; BehaviorIdx < BehaviorNum; BehaviorIdx++)
	{
		if (Behaviors[BehaviorIdx].Behavior && Behaviors[BehaviorIdx].Name == BehaviorName)
		{
			Behaviors[BehaviorIdx].Behavior->DrawDebugControl(
				Drawer, 
				CanvasDrawer,
				InControlObject, 
				BehaviorElement,
				InPoseState, 
				InDatabase, 
				InAutoEncoder, 
				CharacterIdx,
				SequenceIdx, 
				SequenceTime, 
				RangeStart, 
				RangeLength, 
				IdentifierColor);
			break;
		}
	}
}

#endif

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeUncontrolledBehaviorControl(FAnimGenControlObject& InControlObject)
{
	return UAnimGenControls::MakeNullControl(InControlObject);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeIdleBehaviorControl(FAnimGenControlObject& InControlObject)
{
	return UAnimGenControls::MakeNullControl(InControlObject);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeTrajectoryFollowBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FTransformTrajectory& TransformTrajectory,
	const FTransform& RootBoneTransform,
	const int32 InTrajectorySampleNum,
	const float InTrajectoryFutureTime,
	const float InTrajectoryPastTime,
	const FVector InTrajectoryForwardVector)
{
	return UAnimGenControls::MakeTrajectoryControlFromTransformTrajectory(InControlObject, TransformTrajectory, RootBoneTransform, InTrajectorySampleNum, InTrajectoryFutureTime, InTrajectoryPastTime, InTrajectoryForwardVector);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeMoveToTargetBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FVector TargetLocation,
	const FVector TargetDirection,
	const bool bUseTargetDirection,
	const FTransform& RootBoneTransform)
{
	return UAnimGenControls::MakeStructControlFromArrayViews(InControlObject,
		{
			TEXT("Location"),
			TEXT("Direction"),
		},
		{
			UAnimGenControls::MakeLocationControl(InControlObject, TargetLocation, RootBoneTransform),
			bUseTargetDirection ?
				UAnimGenControls::MakeOptionalValidControl(InControlObject, UAnimGenControls::MakeDirectionControl(InControlObject, TargetDirection, RootBoneTransform)) :
				UAnimGenControls::MakeOptionalNullControl(InControlObject),
		});
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeTrajectoryInteractionBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FTransformTrajectory& TransformTrajectory,
	const FTransform& RootBoneTransform,
	const FVector InteractionLocation,
	const FVector InteractionDirection,
	const bool bInteractionActive,
	const int32 InTrajectorySampleNum,
	const float InTrajectoryFutureTime,
	const float InTrajectoryPastTime,
	const FVector InTrajectoryForwardVector)
{
	return UAnimGenControls::MakeStructControlFromArrayViews(InControlObject,
		{
			TEXT("Trajectory"),
			TEXT("Interaction")
		},
		{
			UAnimGenControls::MakeTrajectoryControlFromTransformTrajectory(InControlObject, TransformTrajectory, RootBoneTransform, InTrajectorySampleNum, InTrajectoryFutureTime, InTrajectoryPastTime, InTrajectoryForwardVector),
			bInteractionActive ?
				UAnimGenControls::MakeOptionalValidControl(InControlObject, UAnimGenControls::MakeStructControlFromArrayViews(InControlObject,
					{
						TEXT("Location"),
						TEXT("Direction"),
					},
					{
						UAnimGenControls::MakeLocationControl(InControlObject, InteractionLocation, RootBoneTransform),
						UAnimGenControls::MakeDirectionControl(InControlObject, InteractionDirection, RootBoneTransform),
					})) :
				UAnimGenControls::MakeOptionalNullControl(InControlObject)
		});
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeTaggedBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement& BehaviorControlElement,
	const TArray<FName>& DesiredTags)
{
	return MakeTaggedBehaviorControlFromArrayView(InControlObject, BehaviorControlElement, DesiredTags);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeTaggedBehaviorControlFromArrayView(
	FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement& BehaviorControlElement,
	const TArrayView<const FName> DesiredTags)
{
	return UAnimGenControls::MakeStructControlFromArrayViews(InControlObject,
		{
			TEXT("Behavior"),
			TEXT("Tags"),
		},
		{
			BehaviorControlElement,
			UAnimGenControls::MakeNamedInclusiveDiscreteControlFromArrayView(InControlObject, DesiredTags),
		});
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeValidOptionalBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement& BehaviorControlElement)
{
	return UAnimGenControls::MakeOptionalValidControl(InControlObject, BehaviorControlElement);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeNullOptionalBehaviorControl(
	FAnimGenControlObject& InControlObject)
{
	return UAnimGenControls::MakeOptionalNullControl(InControlObject);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeEncodedBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement& BehaviorControlElement)
{
	return UAnimGenControls::MakeEncodingControl(InControlObject, BehaviorControlElement);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeValidOptionalBehaviorControlOnCondition(
	FAnimGenControlObject& InControlObject,
	const FAnimGenControlObjectElement& BehaviorControlElement,
	const bool bCondition)
{
	return UAnimGenControls::MakeOptionalControlOnCondition(InControlObject, BehaviorControlElement, bCondition);
}

FAnimGenControlObjectElement UAnimGenBehaviorLibrary::MakeMultiBehaviorControl(
	FAnimGenControlObject& InControlObject,
	const FName Behavior,
	const FAnimGenControlObjectElement& BehaviorControlElement)
{
	return UAnimGenControls::MakeExclusiveUnionControl(InControlObject, Behavior, UAnimGenControls::MakeEncodingControl(InControlObject, BehaviorControlElement));
}

FVector UAnimGenBehaviorLibrary::ProjectOntoFloor(const FVector Location, const float FloorHeight)
{
	return FVector(Location.X, Location.Y, FloorHeight);
}

FVector UAnimGenBehaviorLibrary::ClampToMaxDistanceFromVector(
	const FVector InVector,
	const FVector FromVector,
	const float MaxDistance)
{
	const FVector Offset = InVector - FromVector;
	const float SqDist = Offset.SquaredLength();

	if (SqDist > MaxDistance * MaxDistance)
	{
		return FromVector + MaxDistance * Offset.GetSafeNormal();
	}
	else
	{
		return InVector;
	}
}

void UAnimGenBehaviorLibrary::ApplyStickDeadzoneAndSensitivity(
	FVector& OutAxis,
	FVector& OutDirection,
	float& OutLength,
	bool& bOutNotInDeadzone,
	const FVector Axis,
	const float Deadzone,
	const float Sensitivity)
{
	const float SafeDeadzone = FMath::Clamp(Deadzone, 0.0f, 1.0f - UE_SMALL_NUMBER);

	const float AxisLength = Axis.Length();

	if (AxisLength < SafeDeadzone)
	{
		OutAxis = FVector::ZeroVector;
		OutDirection = FVector::ForwardVector;
		OutLength = 0.0f;
		bOutNotInDeadzone = false;
	}
	else
	{
		OutDirection = Axis.GetUnsafeNormal();
		OutLength = FMath::Pow((AxisLength - SafeDeadzone) / (1.0f - SafeDeadzone), FMath::Max(Sensitivity, UE_SMALL_NUMBER));
		OutAxis = OutLength * OutDirection;
		bOutNotInDeadzone = true;
	}
}

bool UAnimGenBehaviorLibrary::IsStickNotInDeadzone(const FVector Axis, const float Deadzone)
{
	return  Axis.Length() >= FMath::Clamp(Deadzone, 0.0f, 1.0f - UE_SMALL_NUMBER);
}

bool UAnimGenBehaviorLibrary::IsStickInDeadzone(const FVector Axis, const float Deadzone)
{
	return  Axis.Length() < FMath::Clamp(Deadzone, 0.0f, 1.0f - UE_SMALL_NUMBER);
}

float UAnimGenBehaviorLibrary::ApplyTriggerSensitivity(const float Trigger, const float Sensitivity)
{
	return  FMath::Pow(Trigger, FMath::Max(Sensitivity, UE_SMALL_NUMBER));
}

void UAnimGenBehaviorLibrary::UpdateDesiredVelocityFromAxis(
	FVector& OutDesiredVelocity,
	const FVector Axis,
	const float VelocitySpeed,
	const float Deadzone,
	const float Sensitivity)
{
	FVector OutAxis = FVector::ZeroVector;
	FVector OutDirection = FVector::ForwardVector;
	float OutLength = 0.0f;
	bool bOutNotInDeadZone = false;

	ApplyStickDeadzoneAndSensitivity(
		OutAxis,
		OutDirection,
		OutLength,
		bOutNotInDeadZone,
		Axis,
		Deadzone,
		Sensitivity);

	// Desired Velocity

	OutDesiredVelocity = bOutNotInDeadZone ? VelocitySpeed * OutAxis : FVector::ZeroVector;
}

FRotator UAnimGenBehaviorLibrary::DesiredDirectionToRotation(const FVector Direction)
{
	return FVector(Direction.Y, -Direction.X, 0.0f).ToOrientationRotator();
}

FQuat UAnimGenBehaviorLibrary::DesiredDirectionToRotationQuat(const FVector Direction)
{
	return FVector(Direction.Y, -Direction.X, 0.0f).ToOrientationQuat();
}

void UAnimGenBehaviorLibrary::UpdateDesiredRotationFromAxis(
	FRotator& OutDesiredRotation,
	const FVector Axis,
	const float Deadzone,
	const float Sensitivity)
{
	FVector OutAxis = FVector::ZeroVector;
	FVector OutDirection = FVector::ForwardVector;
	float OutLength = 0.0f;
	bool bOutNotInDeadZone = false;

	ApplyStickDeadzoneAndSensitivity(
		OutAxis,
		OutDirection,
		OutLength,
		bOutNotInDeadZone,
		Axis,
		Deadzone,
		Sensitivity);

	if (bOutNotInDeadZone)
	{
		OutDesiredRotation = DesiredDirectionToRotation(OutDirection);
	}
}

void UAnimGenBehaviorLibrary::UpdateDesiredVelocityAndRotationFromAxes(
	FVector& OutDesiredVelocity,
	FRotator& OutDesiredRotation,
	const FVector VelocityAxis,
	const FVector FacingAxis,
	const float VelocitySpeed,
	const float Deadzone,
	const float Sensitivity)
{
	// Velocity Stick

	FVector OutVelocityAxis = FVector::ZeroVector;
	FVector OutVelocityDirection = FVector::ForwardVector;
	float OutVelocityLength = 0.0f;
	bool bOutVelocityNotInDeadZone = false;

	ApplyStickDeadzoneAndSensitivity(
		OutVelocityAxis,
		OutVelocityDirection,
		OutVelocityLength,
		bOutVelocityNotInDeadZone,
		VelocityAxis,
		Deadzone,
		Sensitivity);

	// Facing Stick

	FVector OutFacingAxis = FVector::ZeroVector;
	FVector OutFacingDirection = FVector::ForwardVector;
	float OutFacingLength = 0.0f;
	bool bOutFacingNotInDeadZone = false;

	ApplyStickDeadzoneAndSensitivity(
		OutFacingAxis,
		OutFacingDirection,
		OutFacingLength,
		bOutFacingNotInDeadZone,
		FacingAxis,
		Deadzone,
		Sensitivity);

	// Desired Velocity

	OutDesiredVelocity = VelocitySpeed * OutVelocityAxis;

	// Desired Rotation

	if (bOutFacingNotInDeadZone)
	{
		OutDesiredRotation = DesiredDirectionToRotation(OutFacingDirection);
	}
	else if (bOutVelocityNotInDeadZone)
	{
		OutDesiredRotation = DesiredDirectionToRotation(OutVelocityDirection);
	}
}

void UAnimGenBehaviorLibrary::UpdateDesiredVelocityAndRotationFromAxisAndCameraTrigger(
	FVector& OutDesiredVelocity,
	FRotator& OutDesiredRotation,
	const FRotator CameraYawRotation,
	const FVector VelocityAxis,
	const float FacingTrigger,
	const float MinimumSpeed,
	const float MaximumSpeed,
	const float Deadzone,
	const float Sensitivity)
{
	FVector OutVelocityAxis = FVector::ZeroVector;
	FVector OutVelocityDirection = FVector::ForwardVector;
	float OutVelocityLength = 0.0f;
	bool bOutVelocityNotInDeadZone = false;

	ApplyStickDeadzoneAndSensitivity(
		OutVelocityAxis,
		OutVelocityDirection,
		OutVelocityLength,
		bOutVelocityNotInDeadZone,
		CameraYawRotation.RotateVector(VelocityAxis),
		Deadzone,
		Sensitivity);

	if (FacingTrigger > 0.0f)
	{
		if (bOutVelocityNotInDeadZone)
		{
			const FQuat CameraRotation = CameraYawRotation.RotateVector(-FVector::RightVector).ToOrientationQuat();
			const FQuat FacingRotation = DesiredDirectionToRotationQuat(OutVelocityDirection);

			OutDesiredVelocity = FMath::Lerp(MinimumSpeed, MaximumSpeed, OutVelocityLength) * OutVelocityDirection;
			OutDesiredRotation = FQuat::Slerp(FacingRotation, CameraRotation, FacingTrigger).Rotator();
		}
		else
		{
			OutDesiredVelocity = FVector::ZeroVector;
			OutDesiredRotation = CameraYawRotation.RotateVector(-FVector::RightVector).ToOrientationRotator();
		}
	}
	else if (bOutVelocityNotInDeadZone)
	{
		OutDesiredVelocity = FMath::Lerp(MinimumSpeed, MaximumSpeed, OutVelocityLength) * OutVelocityDirection;
		OutDesiredRotation = DesiredDirectionToRotation(OutVelocityDirection);
	}
	else
	{
		OutDesiredVelocity = FVector::ZeroVector;
	}
}

int32 UAnimGenBehaviorLibrary::FindNearestIndexInFloatArray(const TArray<float>& Values, const float Value)
{
	if (Values.IsEmpty())
	{
		return INDEX_NONE;
	}

	const int32 Num = Values.Num();

	float BestDist = UE_MAX_FLT;
	int32 BestIndex = INDEX_NONE;
	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		const float Dist = FMath::Abs(Values[Idx] - Value);
		if (Dist < BestDist)
		{
			BestDist = Dist;
			BestIndex = Idx;
		}
	}

	return BestIndex;
}

FName UAnimGenBehaviorLibrary::CycleNameInArray(const TArray<FName>& Names, const FName CurrentName, const int32 Steps)
{
	return CycleNameInArrayView(Names, CurrentName, Steps);
}

FName UAnimGenBehaviorLibrary::CycleNameInArrayView(const TArrayView<const FName> Names, const FName CurrentName, const int32 Steps)
{
	int32 Index = Names.Find(CurrentName);
	if (Index != INDEX_NONE)
	{
		return Names[FMath::WrapExclusive(Index + Steps, 0, Names.Num())];
	}

	return NAME_None;
}

bool UAnimGenBehaviorLibrary::LocationDifferenceBelowThreshold(
	const FVector CurrentLocation,
	const FVector TargetLocation,
	const float LocationThreshold)
{
	return FVector::DistSquared(TargetLocation, CurrentLocation) < LocationThreshold * LocationThreshold;
}

bool UAnimGenBehaviorLibrary::LocationAndDirectionDifferenceBelowThreshold(
	const FVector CurrentLocation,
	const FVector CurrentDirection,
	const FVector TargetLocation,
	const FVector TargetDirection,
	const float LocationThreshold,
	const float DirectionThresoldAngle)
{
	const bool bLocationClose = FVector::DistSquared(TargetLocation, CurrentLocation) < LocationThreshold * LocationThreshold;
	const bool bDirectionClose = TargetDirection.IsZero() || (FMath::Acos(CurrentDirection.GetSafeNormal().Dot(TargetDirection.GetSafeNormal())) < FMath::DegreesToRadians(DirectionThresoldAngle));
	return bLocationClose && bDirectionClose;
}

FVector UAnimGenBehaviorLibrary::ApplySimpleAvoidanceToVelocity(
	const FVector Velocity,
	const TArray<AActor*>& Actors,
	const AActor* SelfActor,
	const float MinAvoidanceRadius,
	const float MaxAvoidanceRadius,
	const float AvoidanceVelocity)
{
	return ApplySimpleAvoidanceToVelocityArrayView(Velocity, Actors, SelfActor, MinAvoidanceRadius, MaxAvoidanceRadius, AvoidanceVelocity);
}

FVector UAnimGenBehaviorLibrary::ApplySimpleAvoidanceToVelocityArrayView(
	const FVector Velocity,
	const TArrayView<AActor* const> Actors,
	const AActor* SelfActor,
	const float MinAvoidanceRadius,
	const float MaxAvoidanceRadius,
	const float AvoidanceVelocity)
{
	if (!SelfActor) { return Velocity; }

	const FVector Location = SelfActor->GetActorLocation();

	FVector OutVelocity = Velocity;

	const int32 ActorNum = Actors.Num();
	for (int32 ActorIdx = 0; ActorIdx < ActorNum; ActorIdx++)
	{
		if (Actors[ActorIdx] == SelfActor) { continue; }

		const FVector ActorLocation = Actors[ActorIdx]->GetActorLocation();

		const float Distance = FVector::Distance(Location, ActorLocation);
		const float AvoidanceForce = FMath::Clamp((Distance - MinAvoidanceRadius) / FMath::Max(MaxAvoidanceRadius - MinAvoidanceRadius, UE_SMALL_NUMBER), 0.0f, 1.0);

		if (AvoidanceForce < 1.0f)
		{
			const FVector AvoidanceDirection = (Location - ActorLocation).GetSafeNormal();

			OutVelocity = FMath::Lerp(OutVelocity, AvoidanceDirection * AvoidanceVelocity, 1.0f - AvoidanceForce);
		}
	}

	return OutVelocity;
}

bool UAnimGenBehaviorLibrary::CanLookAtActor(
	const FVector LookAtDirection,
	const AActor* Actor,
	const AActor* SelfActor,
	const float DistanceThreshold,
	const float DotAngleThreshold)
{
	if (!Actor || !SelfActor) { return false; }

	const FVector ActorLocation = Actor->GetActorLocation();
	const FVector SelfActorLocation = SelfActor->GetActorLocation();

	return Actor != SelfActor &&
		FVector::Distance(ActorLocation, SelfActorLocation) < DistanceThreshold &&
		(ActorLocation - SelfActorLocation).GetSafeNormal().Dot(LookAtDirection.GetSafeNormal()) > DotAngleThreshold;
}

AActor* UAnimGenBehaviorLibrary::FindLookAtActor(
	const FVector LookAtDirection,
	const TArray<AActor*>& Actors,
	const AActor* SelfActor,
	const float DistanceThreshold,
	const float DotAngleThreshold)
{
	return FindLookAtActorArrayView(LookAtDirection, Actors, SelfActor, DistanceThreshold, DotAngleThreshold);
}

AActor* UAnimGenBehaviorLibrary::FindLookAtActorArrayView(
	const FVector LookAtDirection,
	const TArray<AActor*>& Actors,
	const AActor* SelfActor,
	const float DistanceThreshold,
	const float DotAngleThreshold)
{
	if (!SelfActor) { return nullptr; }

	const int32 ActorNum = Actors.Num();
	for (int32 ActorIdx = 0; ActorIdx < ActorNum; ActorIdx++)
	{
		if (CanLookAtActor(LookAtDirection, Actors[ActorIdx], SelfActor, DistanceThreshold, DotAngleThreshold))
		{
			return Actors[ActorIdx];
		}
	}

	return nullptr;
}
