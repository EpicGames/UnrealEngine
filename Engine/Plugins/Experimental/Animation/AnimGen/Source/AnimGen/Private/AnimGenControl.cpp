// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenControl.h"

#include "AnimDatabase.h"
#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"
#include "AnimDatabasePose.h"

#include "AnimGenLog.h"

#include "DrawDebugLibrary.h"

#include "LearningRandom.h"
#include "LearningFrameAttribute.h"

#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "BonePose.h"
#include "UObject/Package.h"
#include "Animation/TrajectoryTypes.h"
#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGenControl)

/** To avoid spamming the log we define a macro which only logs once */
#define UE_ANIMGEN_LOG_ONCE(...) UE_CALL_ONCE([&] { UE_LOGFMT(__VA_ARGS__); });

namespace UE::AnimGen::Control::Private
{
	static inline bool ContainsDuplicates(const TArrayView<const int32> Indices)
	{
		TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<32>> IndicesSet;
		IndicesSet.Append(Indices);
		return Indices.Num() != IndicesSet.Num();
	}

	static inline bool ContainsDuplicates(const TArrayView<const FName> ElementNames)
	{
		TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<32>> ElementNameSet;
		ElementNameSet.Append(ElementNames);
		return ElementNames.Num() != ElementNameSet.Num();
	}

	static inline Learning::Observation::EEncodingActivationFunction GetEncodingActivationFunction(const EAnimGenActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case EAnimGenActivationFunction::ReLU: return Learning::Observation::EEncodingActivationFunction::ReLU;
		case EAnimGenActivationFunction::ELU: return Learning::Observation::EEncodingActivationFunction::ELU;
		case EAnimGenActivationFunction::TanH: return Learning::Observation::EEncodingActivationFunction::TanH;
		case EAnimGenActivationFunction::GELU: return Learning::Observation::EEncodingActivationFunction::GELU;
		default: checkNoEntry(); return Learning::Observation::EEncodingActivationFunction::ReLU;
		}
	}

	static inline const TCHAR* GetObservationTypeString(const Learning::Observation::EType ObservationType)
	{
		switch (ObservationType)
		{
		case  Learning::Observation::EType::Null: return TEXT("Null");
		case  Learning::Observation::EType::Continuous: return TEXT("Continuous");
		case  Learning::Observation::EType::DiscreteExclusive: return TEXT("DiscreteExclusive");
		case  Learning::Observation::EType::DiscreteInclusive: return TEXT("DiscreteInclusive");
		case  Learning::Observation::EType::NamedDiscreteExclusive: return TEXT("NamedDiscreteExclusive");
		case  Learning::Observation::EType::NamedDiscreteInclusive: return TEXT("NamedDiscreteInclusive");
		case  Learning::Observation::EType::And: return TEXT("Struct");
		case  Learning::Observation::EType::OrExclusive: return TEXT("ExclusiveUnion");
		case  Learning::Observation::EType::OrInclusive: return TEXT("InclusiveUnion");
		case  Learning::Observation::EType::Array: return TEXT("StaticArray");
		case  Learning::Observation::EType::Set: return TEXT("Set");
		case  Learning::Observation::EType::Encoding: return TEXT("Encoding");
		default:
			checkNoEntry();
			return TEXT("Unimplemented");
		}
	}

	static bool ValidateControlObjectMatchesSchema(
		const Learning::Observation::FSchema& Schema,
		const Learning::Observation::FSchemaElement SchemaElement,
		const Learning::Observation::FObject& Object,
		const Learning::Observation::FObjectElement ObjectElement)
	{
		// Check Elements are Valid

		if (!Schema.IsValid(SchemaElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Invalid Control Schema Element.");
			return false;
		}

		if (!Object.IsValid(ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Invalid Control Object Element.");
			return false;
		}

		// Check Names Match

		const FName ObservationSchemaElementName = Schema.GetTag(SchemaElement);
		const FName ObservationObjectElementName = Object.GetTag(ObjectElement);

		if (ObservationSchemaElementName != ObservationObjectElementName)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "ValidateControlObjectMatchesSchema: Control name does not match Schema. Expected '{SchemaType}', got '{ObjectType}'.",
				*ObservationSchemaElementName.ToString(), *ObservationObjectElementName.ToString());
		}

		// Check Types Match

		const Learning::Observation::EType ObservationSchemaElementType = Schema.GetType(SchemaElement);
		const Learning::Observation::EType ObservationObjectElementType = Object.GetType(ObjectElement);

		if (ObservationSchemaElementType != ObservationObjectElementType)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' type does not match Schema. Expected type '{SchemaType}', got type '{ObjectType}'.",
				*ObservationSchemaElementName.ToString(),
				GetObservationTypeString(ObservationSchemaElementType),
				GetObservationTypeString(ObservationObjectElementType));
			return false;
		}

		// Type Specific Checks

		switch (ObservationSchemaElementType)
		{
		case Learning::Observation::EType::Null: return true;

		case Learning::Observation::EType::Continuous:
		{
			const int32 SchemaElementSize = Schema.GetContinuous(SchemaElement).Num;
			const int32 ObjectElementSize = Object.GetContinuous(ObjectElement).Values.Num();

			if (SchemaElementSize != ObjectElementSize)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' size does not match Schema. Expected '{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaElementSize,
					ObjectElementSize);
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::DiscreteExclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteExclusive(SchemaElement).Num;
			const int32 ObjectElementIndex = Object.GetDiscreteExclusive(ObjectElement).DiscreteIndex;

			if (ObjectElementIndex < 0 || ObjectElementIndex >= SchemaElementSize)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' index out of range for Schema. Expected '<{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaElementSize,
					ObjectElementIndex);
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::DiscreteInclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteInclusive(SchemaElement).Num;
			const TArrayView<const int32> ObjectElementIndices = Object.GetDiscreteInclusive(ObjectElement).DiscreteIndices;

			if (ObjectElementIndices.Num() > SchemaElementSize)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' too many indices provided. Expected at most '{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaElementSize,
					ObjectElementIndices.Num());
				return false;
			}

			for (int32 SubElementIdx = 0; SubElementIdx < ObjectElementIndices.Num(); SubElementIdx++)
			{
				if (ObjectElementIndices[SubElementIdx] < 0 || ObjectElementIndices[SubElementIdx] >= SchemaElementSize)
				{
					UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' index out of range for Schema. Expected '<{SchemaSize}', got '{ObjectSize}'.",
						*ObservationSchemaElementName.ToString(),
						SchemaElementSize,
						ObjectElementIndices[SubElementIdx]);
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::NamedDiscreteExclusive:
		{
			const Learning::Observation::FSchemaNamedDiscreteExclusiveParameters SchemaParameters = Schema.GetNamedDiscreteExclusive(SchemaElement);
			const Learning::Observation::FObjectNamedDiscreteExclusiveParameters ObjectParameters = Object.GetNamedDiscreteExclusive(ObjectElement);

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' Schema does not include '{ElementName}' control.",
					*ObservationSchemaElementName.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::NamedDiscreteInclusive:
		{
			const Learning::Observation::FSchemaNamedDiscreteInclusiveParameters SchemaParameters = Schema.GetNamedDiscreteInclusive(SchemaElement);
			const Learning::Observation::FObjectNamedDiscreteInclusiveParameters ObjectParameters = Object.GetNamedDiscreteInclusive(ObjectElement);

			if (ObjectParameters.ElementNames.Num() > SchemaParameters.ElementNames.Num())
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' too many sub-controls provided. Expected at most '{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaParameters.ElementNames.Num(),
					ObjectParameters.ElementNames.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.ElementNames.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' Schema does not include '{ElementName}' control.",
						*ObservationSchemaElementName.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::And:
		{
			const Learning::Observation::FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const Learning::Observation::FObjectAndParameters ObjectParameters = Object.GetAnd(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());
			check(ObjectParameters.Elements.Num() == ObjectParameters.ElementNames.Num());

			if (SchemaParameters.Elements.Num() != ObjectParameters.Elements.Num())
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' number of sub-elements does not match Schema. Expected '{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 ObjectElementIdx = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);

				if (ObjectElementIdx == INDEX_NONE)
				{
					UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' does not include '{ElementName}' control required by Schema.",
						*ObservationSchemaElementName.ToString(),
						*SchemaParameters.ElementNames[SchemaElementIdx].ToString());
					return false;
				}

				if (!ValidateControlObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::OrExclusive:
		{
			const Learning::Observation::FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const Learning::Observation::FObjectOrExclusiveParameters ObjectParameters = Object.GetOrExclusive(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' Schema does not include '{ElementName}' control.",
					*ObservationSchemaElementName.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return ValidateControlObjectMatchesSchema(
				Schema,
				SchemaParameters.Elements[SchemaSubElementIdx],
				Object,
				ObjectParameters.Element);
		}

		case Learning::Observation::EType::OrInclusive:
		{
			const Learning::Observation::FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const Learning::Observation::FObjectOrInclusiveParameters ObjectParameters = Object.GetOrInclusive(ObjectElement);

			if (ObjectParameters.Elements.Num() > SchemaParameters.Elements.Num())
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' too many sub-controls provided. Expected at most '{SchemaSize}', got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.Elements.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' Schema does not include '{ElementName}' control.",
						*ObservationSchemaElementName.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}

				if (!ValidateControlObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaSubElementIdx],
					Object,
					ObjectParameters.Elements[ObjectSubElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Array:
		{
			const Learning::Observation::FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const Learning::Observation::FObjectArrayParameters ObjectParameters = Object.GetArray(ObjectElement);

			if (ObjectParameters.Elements.Num() != SchemaParameters.Num)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' array incorrect size. Expected '{SchemaSize}' elements, got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaParameters.Num,
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateControlObjectMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Set:
		{
			const Learning::Observation::FSchemaSetParameters SchemaParameters = Schema.GetSet(SchemaElement);
			const Learning::Observation::FObjectSetParameters ObjectParameters = Object.GetSet(ObjectElement);

			if (ObjectParameters.Elements.Num() > SchemaParameters.MaxNum)
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Control '{ControlName}' set too large. Expected at most '{SchemaSize}' elements, got '{ObjectSize}'.",
					*ObservationSchemaElementName.ToString(),
					SchemaParameters.MaxNum,
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateControlObjectMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Encoding:
		{
			const Learning::Observation::FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const Learning::Observation::FObjectEncodingParameters ObjectParameters = Object.GetEncoding(ObjectElement);

			return ValidateControlObjectMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);
		}

		default:
		{
			checkNoEntry();
			return true;
		}
		}
	}
}

bool FAnimGenControlSchema::IsValid() const
{
	return ObservationSchema.IsValid();
}

bool FAnimGenControlSchema::IsElementValid(const FAnimGenControlSchemaElement& Element) const
{
	return IsValid() && ObservationSchema->Schema.IsValid(Element.SchemaElement);
}

void FAnimGenControlSchema::Invalidate()
{
	ObservationSchema.Reset();
}

bool FAnimGenControlObject::IsValid() const
{
	return ObservationObject.IsValid();
}

bool FAnimGenControlObject::IsElementValid(const FAnimGenControlObjectElement& Element) const
{
	return IsValid() && ObservationObject->Object.IsValid(Element.ObjectElement);
}

void FAnimGenControlObject::Invalidate()
{
	ObservationObject.Reset();
}

bool operator==(const FAnimGenControlSchema& Lhs, const FAnimGenControlSchema& Rhs)
{
	return Lhs.ObservationSchema == Rhs.ObservationSchema;
}

bool FAnimGenControlSchema::Serialize(FArchive& Ar)
{
	static constexpr int32 Magic = 0x4aa878ee;
	static constexpr int32 Version = 0;

	if (Ar.IsSaving())
	{
		int32 SaveMagic = Magic;
		int32 SaveVersion = Version;
		bool bSaveIsValid = IsValid();

		Ar << SaveMagic;
		Ar << SaveVersion;
		Ar << bSaveIsValid;
		if (bSaveIsValid)
		{
			UE::AnimGen::FSpinLockScope SchemaLock(ObservationSchema->Lock);
			return ObservationSchema->Schema.Serialize(Ar);
		}
	}
	else if (Ar.IsLoading())
	{
		int64 Offset = Ar.Tell();
		int32 LoadMagic = 0;
		int32 LoadVersion = 0;
		bool bLoadIsValid = true;

		Ar << LoadMagic;
		if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); ObservationSchema.Reset(); return false; }
		Ar << LoadVersion;
		if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); ObservationSchema.Reset(); return false; }
		Ar << bLoadIsValid;
		if (bLoadIsValid)
		{
			if (ObservationSchema)
			{
				UE::AnimGen::FSpinLockScope SchemaLock(ObservationSchema->Lock);
				return ObservationSchema->Schema.Serialize(Ar);
			}
			else
			{
				ObservationSchema = MakeShared<UE::AnimGen::FControlSchema>();
				return ObservationSchema->Schema.Serialize(Ar);
			}
		}
		else
		{
			ObservationSchema.Reset();
		}
	}

	return true;
}

bool operator==(const FAnimGenControlObject& Lhs, const FAnimGenControlObject& Rhs)
{
	return Lhs.ObservationObject == Rhs.ObservationObject;
}

bool FAnimGenControlObject::Serialize(FArchive& Ar)
{
	static constexpr int32 Magic = 0x953f2ad0;
	static constexpr int32 Version = 0;

	if (Ar.IsSaving())
	{
		int32 SaveMagic = Magic;
		int32 SaveVersion = Version;
		bool bSaveIsValid = IsValid();

		Ar << SaveMagic;
		Ar << SaveVersion;
		Ar << bSaveIsValid;
		if (bSaveIsValid)
		{
			UE::AnimGen::FSpinLockScope ObjectLock(ObservationObject->Lock);
			return ObservationObject->Object.Serialize(Ar);
		}
	}
	else if (Ar.IsLoading())
	{
		int64 Offset = Ar.Tell();
		int32 LoadMagic = 0;
		int32 LoadVersion = 0;
		bool bLoadIsValid = true;

		Ar << LoadMagic;
		if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); ObservationObject.Reset(); return false; }
		Ar << LoadVersion;
		if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); ObservationObject.Reset(); return false; }
		Ar << bLoadIsValid;
		if (bLoadIsValid)
		{
			if (ObservationObject)
			{
				UE::AnimGen::FSpinLockScope ObjectLock(ObservationObject->Lock);
				return ObservationObject->Object.Serialize(Ar);
			}
			else
			{
				ObservationObject = MakeShared<UE::AnimGen::FControlObject>();
				return ObservationObject->Object.Serialize(Ar);
			}
		}
		else
		{
			ObservationObject.Reset();
		}
	}

	return true;
}

bool FAnimGenControlSchemaElement::Serialize(FArchive& Ar)
{
	Ar << SchemaElement;
	return true;
}

bool FAnimGenControlObjectElement::Serialize(FArchive& Ar)
{
	Ar << ObjectElement;
	return true;
}

bool operator==(const FAnimGenControlSchemaElement& Lhs, const FAnimGenControlSchemaElement& Rhs)
{
	return Lhs.SchemaElement == Rhs.SchemaElement;
}

bool operator==(const FAnimGenControlObjectElement& Lhs, const FAnimGenControlObjectElement& Rhs)
{
	return Lhs.ObjectElement == Rhs.ObjectElement;
}

uint32 GetTypeHash(const FAnimGenControlSchemaElement& Element)
{
	return GetTypeHash(Element.SchemaElement);
}

uint32 GetTypeHash(const FAnimGenControlObjectElement& Element)
{
	return GetTypeHash(Element.ObjectElement);
}

TArrayView<const FAnimGenControlObjectElement> FAnimGenControlSet::GetControls() const
{
	return Data->Controls;
}

bool FAnimGenControlSet::IsValid() const
{
	return Data.IsValid();
}

void FAnimGenControlSet::Invalidate()
{
	Data.Reset();
}

FAnimGenControlSchema UAnimGenControls::MakeControlSchema()
{
	FAnimGenControlSchema Schema;
	Schema.ObservationSchema = MakeShared<UE::AnimGen::FControlSchema>();
	Schema.ObservationSchema->Schema.SetGeneration(FGuid::NewGuid().A);
	return Schema;
}

FAnimGenControlObject UAnimGenControls::MakeControlObject()
{
	FAnimGenControlObject Object;
	Object.ObservationObject = MakeShared<UE::AnimGen::FControlObject>();
	Object.ObservationObject->Object.SetGeneration(FGuid::NewGuid().A);
	return Object;
}

void UAnimGenControls::ResetControlSchema(FAnimGenControlSchema& Schema)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ResetControlSchema: Invalid Control Schema.");
		return;
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		Schema.ObservationSchema->Schema.Reset();
	}
}

void UAnimGenControls::ResetControlObject(FAnimGenControlObject& Object)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ResetControlObject: Invalid Control Object.");
		return;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		Object.ObservationObject->Object.Reset();
	}
}

bool UAnimGenControls::IsControlSchemaValid(const FAnimGenControlSchema& Schema)
{
	return Schema.IsValid();
}

bool UAnimGenControls::IsControlObjectValid(const FAnimGenControlObject& Object)
{
	return Object.IsValid();
}

bool UAnimGenControls::ValidateControlObjectMatchesSchema(
	const FAnimGenControlSchema& Schema,
	const FAnimGenControlSchemaElement SchemaElement,
	const FAnimGenControlObject& Object,
	const FAnimGenControlObjectElement ObjectElement)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Invalid Control Schema.");
		return false;
	}

	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ValidateControlObjectMatchesSchema: Invalid Control Object.");
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		return UE::AnimGen::Control::Private::ValidateControlObjectMatchesSchema(
			Schema.ObservationSchema->Schema,
			SchemaElement.SchemaElement,
			Object.ObservationObject->Object,
			ObjectElement.ObjectElement);
	}
}


FAnimGenControlSchemaElement UAnimGenControls::SpecifyNullControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNullControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateNull(Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyContinuousControl(FAnimGenControlSchema& Schema, const int32 Size, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyContinuousControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Size < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyContinuousControl: Invalid Continuous Control Size '{Size}'.", Size);
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateContinuous({ Size, {}, {}, UE::Learning::Observation::ENormalization::AutoDimensionAverage}, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedExclusiveDiscreteControl(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag)
{
	return SpecifyNamedExclusiveDiscreteControlFromArrayView(Schema, ElementNames, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedExclusiveDiscreteControlFromArrayView(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedExclusiveDiscreteControlFromArrayView: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (ElementNames.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyNamedExclusiveDiscreteControlFromArrayView: Specifying zero-sized Discrete Control.");
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedExclusiveDiscreteControlFromArrayView: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateNamedDiscreteExclusive({ ElementNames }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedInclusiveDiscreteControl(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag)
{
	return SpecifyNamedInclusiveDiscreteControlFromArrayView(Schema, ElementNames, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedInclusiveDiscreteControlFromArrayView(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedInclusiveDiscreteControlFromArrayView: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedInclusiveDiscreteControlFromArrayView: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateNamedDiscreteInclusive({ ElementNames }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyExclusiveDiscreteControl(FAnimGenControlSchema& Schema, const int32 Size, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveDiscreteControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Size < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveDiscreteControl: Invalid Discrete Control Size '{Size}'.", Size);
		return FAnimGenControlSchemaElement();
	}

	if (Size == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyExclusiveDiscreteControl: Specifying zero-sized Discrete Control.");
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateDiscreteExclusive({ Size }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyInclusiveDiscreteControl(FAnimGenControlSchema& Schema, const int32 Size, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveDiscreteControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Size < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveDiscreteControl: Invalid Discrete Control Size '{Size}'.", Size);
		return FAnimGenControlSchemaElement();
	}

	if (Size == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyInclusiveDiscreteControl: Specifying zero-sized Discrete Control.");
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateDiscreteInclusive({ Size }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyCountControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 1, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyStructControl(FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyStructControl: Specifying zero-sized Struct Control.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(Elements.Num());
	SubElementNames.Empty(Elements.Num());
	SubElements.Empty(Elements.Num());

	int32 Index = 0;
	for (const TPair<FName, FAnimGenControlSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyStructControlFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyStructControlFromArrays(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const FName Tag)
{
	return SpecifyStructControlFromArrayViews(Schema, ElementNames, Elements, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyStructControlFromArrayViews(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStructControlFromArrayViews: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyStructControlFromArrayViews: Specifying zero-sized Struct Control.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStructControlFromArrayViews: Number of elements ({ElementNum}) must match number of names ({NameNum}).", Elements.Num(), ElementNames.Num());
		return FAnimGenControlSchemaElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStructControlFromArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FAnimGenControlSchemaElement& Element : Elements)
	{
		{
			UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

			if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStructControlFromArrayViews: Invalid Control Schema Element.");
				return FAnimGenControlSchemaElement();
			}
		}

		SubElements.Add(Element.SchemaElement);
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateAnd({ ElementNames, SubElements }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyExclusiveUnionControl(FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const int32 EncodingSize, const FName Tag)
{
	if (EncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControl: Invalid Control EncodingSize '{Size}' - must be greater than zero.", EncodingSize);
		return FAnimGenControlSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyExclusiveUnionControl: Specifying zero-sized Exclusive Union Control.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(Elements.Num());
	SubElementNames.Empty(Elements.Num());
	SubElements.Empty(Elements.Num());

	int32 Index = 0;
	for (const TPair<FName, FAnimGenControlSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyExclusiveUnionControlFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, EncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyExclusiveUnionControlFromArrays(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const int32 EncodingSize, const FName Tag)
{
	return SpecifyExclusiveUnionControlFromArrayViews(Schema, ElementNames, Elements, EncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyExclusiveUnionControlFromArrayViews(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const int32 EncodingSize, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControlFromArrayViews: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (EncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControlFromArrayViews: Invalid Control EncodingSize '{Size}' - must be greater than zero.", EncodingSize);
		return FAnimGenControlSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyExclusiveUnionControlFromArrayViews: Specifying zero-sized Exclusive Union Control.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControlFromArrayViews: Number of elements ({ElementNum}) must match number of names ({NameNum}).", Elements.Num(), ElementNames.Num());
		return FAnimGenControlSchemaElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControlFromArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FAnimGenControlSchemaElement& Element : Elements)
	{
		{
			UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

			if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyExclusiveUnionControlFromArrayViews: Invalid Control Schema Element.");
				return FAnimGenControlSchemaElement();
			}
		}

		SubElements.Add(Element.SchemaElement);
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateOrExclusive({ ElementNames, SubElements, EncodingSize }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyInclusiveUnionControl(FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControl: Invalid Control Parameters: AttentionEncodingSize: {AttentionSize}, AttentionHeadNum: {HeadNum}, ValueEncodingSize: {ValueSize} - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FAnimGenControlSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyInclusiveUnionControl: Specifying zero-sized Inclusive Union Control.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	int32 Index = 0;
	for (const TPair<FName, FAnimGenControlSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FAnimGenControlSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyInclusiveUnionControlFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyInclusiveUnionControlFromArrays(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	return SpecifyInclusiveUnionControlFromArrayViews(Schema, ElementNames, Elements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyInclusiveUnionControlFromArrayViews(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControlFromArrayViews: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControlFromArrayViews: Invalid Control Parameters: AttentionEncodingSize: {AttentionSize}, AttentionHeadNum: {HeadNum}, ValueEncodingSize: {ValueSize} - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FAnimGenControlSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyInclusiveUnionControlFromArrayViews: Specifying zero-sized Inclusive Union Control.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControlFromArrayViews: Number of elements ({ElementNum}) must match number of names ({NameNum}).", Elements.Num(), ElementNames.Num());
		return FAnimGenControlSchemaElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControlFromArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FAnimGenControlSchemaElement& Element : Elements)
	{	
		{
			UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

			if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyInclusiveUnionControlFromArrayViews: Invalid Control Schema Element.");
				return FAnimGenControlSchemaElement();
			}
		}

		SubElements.Add(Element.SchemaElement);
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateOrInclusive({ ElementNames, SubElements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyStaticArrayControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 Num, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStaticArrayControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Num < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStaticArrayControl: Invalid Control Array Num {Num}.", Num);
		return FAnimGenControlSchemaElement();
	}

	if (Num == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifyStaticArrayControl: Specifying zero-sized Static Array Control.");
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

		if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyStaticArrayControl: Invalid Control Schema Element.");
			return FAnimGenControlSchemaElement();
		}
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateArray({ Element.SchemaElement, Num }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifySetControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySetControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (MaxNum < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySetControl: Invalid Control Set MaxNum {MaxNum} - must be greater than or equal to zero.", MaxNum);
		return FAnimGenControlSchemaElement();
	}

	if (MaxNum == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifySetControl: Specifying zero-sized Set Control.");
	}

	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySetControl: Invalid Control Parameters: AttentionEncodingSize: {AttentionSize}, AttentionHeadNum: {HeadNum}, ValueEncodingSize: {ValueSize} - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

		if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySetControl: Invalid Control Schema Element.");
			return FAnimGenControlSchemaElement();
		}
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateSet({ Element.SchemaElement, MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedSparseControl(FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag)
{
	return SpecifyNamedSparseControlFromArrayView(Schema, ElementNames, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyNamedSparseControlFromArrayView(FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedSparseControlFromArrayView: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyNamedSparseControlFromArrayView: Element Names contain duplicates.");
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateNamedSparse({ ElementNames }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifySparseControl(FAnimGenControlSchema& Schema, const int32 Size, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySparseControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (Size < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifySparseControl: Invalid Discrete Control Size '{Size}'.", Size);
		return FAnimGenControlSchemaElement();
	}

	if (Size == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "SpecifySparseControl: Specifying zero-sized Discrete Control.");
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateSparse({ Size }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyPairControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element0, const FAnimGenControlSchemaElement Element1, const FName Tag)
{
	return SpecifyStructControlFromArrayViews(Schema, { TEXT("Key"), TEXT("Value") }, { Element0, Element1 }, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyArrayControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	return SpecifySetControl(Schema, SpecifyPairControl(Schema, SpecifyCountControl(Schema), Element), MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyMapControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement KeyElement, const FAnimGenControlSchemaElement ValueElement, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const FName Tag)
{
	return SpecifySetControl(Schema, SpecifyPairControl(Schema, KeyElement, ValueElement), MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyEnumControl(FAnimGenControlSchema& Schema, const UEnum* Enum, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyEnumControl: Enum is nullptr.");
		return FAnimGenControlSchemaElement();
	}

	TArray<FName, TInlineAllocator<32>> EnumNames;
	EnumNames.SetNumUninitialized(Enum->NumEnums() - 1);
	for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
	{
		EnumNames[EnumIdx] = Enum->GetNameByIndex(EnumIdx);
	}

	return SpecifyNamedExclusiveDiscreteControlFromArrayView(Schema, EnumNames, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyBitmaskControl(FAnimGenControlSchema& Schema, const UEnum* Enum, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyBitmaskControl: Enum is nullptr.");
		return FAnimGenControlSchemaElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyBitmaskControl: Too many values in Enum to use as Bitmask ({EnumSize}).", Enum->NumEnums() - 1);
		return FAnimGenControlSchemaElement();
	}

	TArray<FName, TInlineAllocator<32>> EnumNames;
	EnumNames.SetNumUninitialized(Enum->NumEnums() - 1);
	for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
	{
		EnumNames[EnumIdx] = Enum->GetNameByIndex(EnumIdx);
	}

	return SpecifyNamedInclusiveDiscreteControlFromArrayView(Schema, EnumNames, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyOptionalControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 EncodingSize, const FName Tag)
{
	return SpecifyExclusiveUnionControlFromArrayViews(Schema, { TEXT("Null"), TEXT("Valid") }, { SpecifyNullControl(Schema), Element }, EncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyEitherControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement A, const FAnimGenControlSchemaElement B, const int32 EncodingSize, const FName Tag)
{
	return SpecifyExclusiveUnionControlFromArrayViews(Schema, { TEXT("A"), TEXT("B") }, { A, B }, EncodingSize, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyEncodingControl(FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 EncodingSize, const int32 HiddenLayerNum, const EAnimGenActivationFunction ActivationFunction, const FName Tag)
{
	if (!Schema.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyEncodingControl: Invalid Control Schema.");
		return FAnimGenControlSchemaElement();
	}

	if (EncodingSize < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyEncodingControl: Invalid Control EncodingSize '{EncodingSize}' - must be greater than zero.", EncodingSize);
		return FAnimGenControlSchemaElement();
	}

	if (HiddenLayerNum < 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyEncodingControl: Invalid Control HiddenLayerNum '{LayerNum}' - must be greater than zero.", HiddenLayerNum);
		return FAnimGenControlSchemaElement();
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);

		if (!Schema.ObservationSchema->Schema.IsValid(Element.SchemaElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "SpecifyEncodingControl: Invalid Control Schema Element.");
			return FAnimGenControlSchemaElement();
		}
	}

	{
		UE::AnimGen::FSpinLockScope SchemaLock(Schema.ObservationSchema->Lock);
		return { Schema.ObservationSchema->Schema.CreateEncoding({ Element.SchemaElement, EncodingSize, HiddenLayerNum, UE::AnimGen::Control::Private::GetEncodingActivationFunction(ActivationFunction) }, Tag) };
	}
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyBoolControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyNamedExclusiveDiscreteControl(Schema, { TEXT("true"), TEXT("false") }, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyFloatControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 1, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyLocationControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 3, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyRotationControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 6, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyScaleControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 3, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyTransformControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyStructControlFromArrayViews(Schema,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
		{
			SpecifyLocationControl(Schema),
			SpecifyRotationControl(Schema),
			SpecifyScaleControl(Schema)
		},
		Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyAngleControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 2, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyLinearVelocityControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 3, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyAngularVelocityControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 3, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyDirectionControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 3, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyTimeControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 1, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyLocationsStaticArrayControl(FAnimGenControlSchema& Schema, const int32 Num, const FName Tag)
{
	return SpecifyStaticArrayControl(Schema, SpecifyLocationControl(Schema), Num, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyLinearVelocitiesStaticArrayControl(FAnimGenControlSchema& Schema, const int32 Num, const FName Tag)
{
	return SpecifyStaticArrayControl(Schema, SpecifyLinearVelocityControl(Schema), Num, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyDirectionsStaticArrayControl(FAnimGenControlSchema& Schema, const int32 Num, const FName Tag)
{
	return SpecifyStaticArrayControl(Schema, SpecifyDirectionControl(Schema), Num, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyPoseControl(FAnimGenControlSchema& Schema, const int32 BoneNum, const FName Tag)
{
	return SpecifyStructControlFromArrayViews(
		Schema,
		{
			TEXT("Locations"),
			TEXT("LinearVelocities"),
		},
		{
			SpecifyLocationsStaticArrayControl(Schema, BoneNum),
			SpecifyLinearVelocitiesStaticArrayControl(Schema, BoneNum),
		},
		Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyTrajectoryControl(FAnimGenControlSchema& Schema, const int32 TrajectorySampleNum, const FName Tag)
{
	return SpecifyStructControlFromArrayViews(
		Schema,
		{
			TEXT("Locations"),
			TEXT("Directions"),
		},
		{
			SpecifyLocationsStaticArrayControl(Schema, TrajectorySampleNum),
			SpecifyDirectionsStaticArrayControl(Schema, TrajectorySampleNum),
		},
		Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyLocationTrajectoryControl(FAnimGenControlSchema& Schema, const int32 TrajectorySampleNum, const FName Tag)
{
	return SpecifyLocationsStaticArrayControl(Schema, TrajectorySampleNum, Tag);
}

FAnimGenControlSchemaElement UAnimGenControls::SpecifyEventControl(FAnimGenControlSchema& Schema, const FName Tag)
{
	return SpecifyContinuousControl(Schema, 2, Tag);
}


FAnimGenControlObjectElement UAnimGenControls::MakeNullControl(FAnimGenControlObject& Object, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNullControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateNull(Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeContinuousControl(FAnimGenControlObject& Object, const TArray<float>& Values, const FName Tag)
{
	return MakeContinuousControlFromArrayView(Object, Values, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeContinuousControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const float> Values, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeContinuousControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (Values.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "MakeContinuousControlFromArrayView: Creating zero-sized Continuous Control.");
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateContinuous({ Values }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeNamedExclusiveDiscreteControl(FAnimGenControlObject& Object, const FName ElementName, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNamedExclusiveDiscreteControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateNamedDiscreteExclusive({ ElementName }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeNamedInclusiveDiscreteControl(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const FName Tag)
{
	return MakeNamedInclusiveDiscreteControlFromArrayView(Object, ElementNames, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeNamedInclusiveDiscreteControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlFromArrayView: Element Names contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateNamedDiscreteInclusive({ ElementNames }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeExclusiveDiscreteControl(FAnimGenControlObject& Object, const int32 DiscreteIndex, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeExclusiveDiscreteControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateDiscreteExclusive({ DiscreteIndex }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeInclusiveDiscreteControl(FAnimGenControlObject& Object, const TArray<int32>& DiscreteIndices, const FName Tag)
{
	return MakeInclusiveDiscreteControlFromArrayView(Object, DiscreteIndices, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeInclusiveDiscreteControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const int32> DiscreteIndices, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveDiscreteControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(DiscreteIndices))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveDiscreteControlFromArrayView: Indices contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateDiscreteInclusive({ DiscreteIndices }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeCountControl(FAnimGenControlObject& Object, const int32 Num, const int32 MaxNum, const FName Tag)
{
	if (MaxNum == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeCountControl: MaxNum must not be zero for Count Control.");
		return FAnimGenControlObjectElement();
	}

	const float Encoded = (float)Num / (float)MaxNum;

	return MakeContinuousControlFromArrayView(Object, { Encoded }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeStructControl(FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "MakeStructControl: Creating zero-sized Struct Control.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	for (const TPair<FName, FAnimGenControlObjectElement>& Element : Elements)
	{
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
	}

	return MakeStructControlFromArrayViews(Object, SubElementNames, SubElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeStructControlFromArrays(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	return MakeStructControlFromArrayViews(Object, ElementNames, Elements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeStructControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStructControlFromArrayViews: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "MakeStructControlFromArrayViews: Creating zero-sized Struct Control.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStructControlFromArrayViews: Number of elements ({ElementNum}) must match number of names ({NameNum}).", Elements.Num(), ElementNames.Num());
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStructControlFromArrayViews: Element Names contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
		SubElements.Empty(Elements.Num());

		for (const FAnimGenControlObjectElement& Element : Elements)
		{
			if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStructControlFromArrayViews: Invalid Control Object Element.");
				return FAnimGenControlObjectElement();
			}

			SubElements.Add(Element.ObjectElement);
		}

		return { Object.ObservationObject->Object.CreateAnd({ ElementNames, SubElements }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeExclusiveUnionControl(FAnimGenControlObject& Object, const FName ElementName, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeExclusiveUnionControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeExclusiveUnionControl: Invalid Control Object Element.");
			return FAnimGenControlObjectElement();
		}

		return { Object.ObservationObject->Object.CreateOrExclusive({ ElementName, Element.ObjectElement }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeInclusiveUnionControl(FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	const int32 SubElementNum = Elements.Num();

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	for (const TPair<FName, FAnimGenControlObjectElement>& Element : Elements)
	{
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
	}

	return MakeInclusiveUnionControlFromArrayViews(Object, SubElementNames, SubElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeInclusiveUnionControlFromArrays(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	return MakeInclusiveUnionControlFromArrayViews(Object, ElementNames, Elements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeInclusiveUnionControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveUnionControlFromArrayViews: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveUnionControlFromArrayViews: Number of elements ({ElementNum}) must match number of names ({NameNum}).", Elements.Num(), ElementNames.Num());
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveUnionControlFromArrayViews: Element Names contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
		SubElements.Empty(Elements.Num());

		for (const FAnimGenControlObjectElement& Element : Elements)
		{
			if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeInclusiveUnionControlFromArrayViews: Invalid Control Object Element.");
				return FAnimGenControlObjectElement();
			}

			SubElements.Add(Element.ObjectElement);
		}

		return { Object.ObservationObject->Object.CreateOrInclusive({ ElementNames, SubElements }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeStaticArrayControl(FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	return MakeStaticArrayControlFromArrayView(Object, Elements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeStaticArrayControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStaticArrayControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "MakeStaticArrayControlFromArrayView: Creating zero-sized Static Array Control.");
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
		SubElements.Empty(Elements.Num());

		for (const FAnimGenControlObjectElement& Element : Elements)
		{
			if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeStaticArrayControlFromArrayView: Invalid Control Object Element.");
				return FAnimGenControlObjectElement();
			}

			SubElements.Add(Element.ObjectElement);
		}

		return { Object.ObservationObject->Object.CreateArray({ SubElements }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeSetControl(FAnimGenControlObject& Object, const TSet<FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSetControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
		SubElements.Empty(Elements.Num());

		for (const FAnimGenControlObjectElement& Element : Elements)
		{
			if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSetControl: Invalid Control Object Element.");
				return FAnimGenControlObjectElement();
			}

			SubElements.Add(Element.ObjectElement);
		}

		return { Object.ObservationObject->Object.CreateSet({ SubElements }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeSetControlFromArray(FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag)
{
	return MakeSetControlFromArrayView(Object, Elements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeSetControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSetControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
		SubElements.Empty(Elements.Num());

		for (const FAnimGenControlObjectElement& Element : Elements)
		{
			if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSetControlFromArrayView: Invalid Control Object Element.");
				return FAnimGenControlObjectElement();
			}

			SubElements.Add(Element.ObjectElement);
		}

		return { Object.ObservationObject->Object.CreateSet({ SubElements }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeNamedSparseControl(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<float>& ElementValues, const FName Tag)
{
	return MakeNamedSparseControlFromArrayView(Object, ElementNames, ElementValues, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeNamedSparseControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const float> ElementValues, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNamedSparseControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (ElementNames.Num() != ElementValues.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSparseControlFromArrayView: ElementNames must be the same size as ElementValues. Got {Indices} and {Values}.", ElementNames.Num(), ElementValues.Num());
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeNamedSparseControlFromArrayView: Element Names contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateNamedSparse({ ElementNames, ElementValues }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeSparseControl(FAnimGenControlObject& Object, const TArray<int32>& Indices, const TArray<float>& Values, const FName Tag)
{
	return MakeSparseControlFromArrayView(Object, Indices, Values, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeSparseControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const int32> Indices, const TArrayView<const float> Values, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSparseControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(Indices))
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSparseControlFromArrayView: Indices contain duplicates.");
		return FAnimGenControlObjectElement();
	}

	if (Indices.Num() != Values.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeSparseControlFromArrayView: Indices must be the same size as Values. Got {Indices} and {Values}.", Indices.Num(), Values.Num());
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);
		return { Object.ObservationObject->Object.CreateSparse({ Indices, Values }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakePairControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Key, const FAnimGenControlObjectElement Value, const FName Tag)
{
	return MakeStructControlFromArrayViews(Object, { TEXT("Key"), TEXT("Value") }, { Key, Value }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeArrayControl(FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const int32 MaxNum, const FName Tag)
{
	return MakeArrayControlFromArrayView(Object, Elements, MaxNum, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeArrayControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const int32 MaxNum, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeArrayControlFromArrayView: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ElementIdx++)
	{
		SubElements.Add(MakePairControl(Object, MakeCountControl(Object, ElementIdx, MaxNum), Elements[ElementIdx]));
	}

	return MakeSetControlFromArrayView(Object, SubElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeMapControl(FAnimGenControlObject& Object, const TMap<FAnimGenControlObjectElement, FAnimGenControlObjectElement>& Map, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeMapControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Map.Num());

	for (TPair<FAnimGenControlObjectElement, FAnimGenControlObjectElement> Item : Map)
	{
		SubElements.Add(MakePairControl(Object, Item.Key, Item.Value));
	}

	return MakeSetControlFromArrayView(Object, SubElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeMapControlFromArrays(FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Keys, const TArray<FAnimGenControlObjectElement>& Values, const FName Tag)
{
	return MakeMapControlFromArrayViews(Object, Keys, Values, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeMapControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Keys, const TArrayView<const FAnimGenControlObjectElement> Values, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeMapControlFromArrayViews: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	if (Keys.Num() != Values.Num())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeMapControlFromArrayViews: Number of keys ({KeyNum}) must match number of values ({ValueNum}).", Keys.Num(), Values.Num());
		return FAnimGenControlObjectElement();
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Keys.Num());

	for (int32 ElementIdx = 0; ElementIdx < Keys.Num(); ElementIdx++)
	{
		SubElements.Add(MakePairControl(Object, Keys[ElementIdx], Values[ElementIdx]));
	}

	return MakeSetControlFromArrayView(Object, SubElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEnumControl(FAnimGenControlObject& Object, const UEnum* Enum, const uint8 EnumValue, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeEnumControl: Enum is nullptr.");
		return FAnimGenControlObjectElement();
	}

	const int32 EnumValueIndex = Enum->GetIndexByValue(EnumValue);

	if (EnumValueIndex == INDEX_NONE || EnumValueIndex < 0 || EnumValueIndex >= Enum->NumEnums() - 1)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeEnumControl: EnumValue {EnumValue} not valid for Enum '{EnumName}'.", EnumValue, *Enum->GetName());
		return FAnimGenControlObjectElement();
	}

	return MakeNamedExclusiveDiscreteControl(Object, Enum->GetNameByIndex(EnumValueIndex), Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeBitmaskControl(FAnimGenControlObject& Object, const UEnum* Enum, const int32 BitmaskValue, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeBitmaskControl: Enum is nullptr.");
		return FAnimGenControlObjectElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeBitmaskControl: Too many values in Enum to use as Bitmask ({EnumSize}).", Enum->NumEnums() - 1);
		return FAnimGenControlObjectElement();
	}

	TArray<FName, TInlineAllocator<32>> Names;
	Names.Reserve(Enum->NumEnums() - 1);

	for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
	{
		if (BitmaskValue & (1 << EnumIdx))
		{
			Names.Add(Enum->GetNameByIndex(EnumIdx));
		}
	}

	return MakeNamedInclusiveDiscreteControlFromArrayView(Object, Names, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeOptionalControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const EAnimGenOptionalControl Option, const FName Tag)
{
	return MakeExclusiveUnionControl(
		Object,
		Option == EAnimGenOptionalControl::Null ? TEXT("Null") : TEXT("Valid"),
		Option == EAnimGenOptionalControl::Null ? MakeNullControl(Object) : Element,
		Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeOptionalNullControl(FAnimGenControlObject& Object, const FName Tag)
{
	return MakeExclusiveUnionControl(Object, TEXT("Null"), MakeNullControl(Object), Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeOptionalValidControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return MakeExclusiveUnionControl(Object, TEXT("Valid"), Element, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeOptionalControlOnCondition(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const bool bCondition, const FName Tag)
{
	return bCondition ? MakeOptionalValidControl(Object, Element, Tag) : MakeOptionalNullControl(Object, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeOptionalControlOnConditionWithFunction(FAnimGenControlObject& Object, TFunctionRef<FAnimGenControlObjectElement(FAnimGenControlObject&)> ElementFunction, const bool bCondition, const FName Tag)
{
	return bCondition ? MakeOptionalValidControl(Object, ElementFunction(Object), Tag) : MakeOptionalNullControl(Object, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEitherControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const EAnimGenEitherControl Either, const FName Tag)
{
	return MakeExclusiveUnionControl(Object, Either == EAnimGenEitherControl::A ? TEXT("A") : TEXT("B"), Element, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEitherAControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement A, const FName Tag)
{
	return MakeExclusiveUnionControl(Object, TEXT("A"), A, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEitherBControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement B, const FName Tag)
{
	return MakeExclusiveUnionControl(Object, TEXT("B"), B, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEncodingControl(FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeEncodingControl: Invalid Control Object.");
		return FAnimGenControlObjectElement();
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "MakeEncodingControl: Invalid Control Object Element.");
			return FAnimGenControlObjectElement();
		}

		return { Object.ObservationObject->Object.CreateEncoding({ Element.ObjectElement }, Tag) };
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeBoolControl(FAnimGenControlObject& Object, const bool bValue, const FName Tag)
{
	return MakeNamedExclusiveDiscreteControl(Object, bValue ? TEXT("true") : TEXT("false"), Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeFloatControl(FAnimGenControlObject& Object, const float Value, const FName Tag)
{
	return MakeContinuousControlFromArrayView(Object, { Value }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationControl(FAnimGenControlObject& Object, const FVector Location, const FTransform& RelativeTransform, const FName Tag)
{
	const FVector LocalLocation = RelativeTransform.InverseTransformPosition(Location);

	return MakeContinuousControlFromArrayView(Object, {
		(float)LocalLocation.X,
		(float)LocalLocation.Y,
		(float)LocalLocation.Z,
		}, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeRotationControl(FAnimGenControlObject& Object, const FRotator Rotation, const FRotator RelativeRotation, const FName Tag)
{
	return MakeRotationControlFromQuat(Object, FQuat::MakeFromRotator(Rotation), FQuat::MakeFromRotator(RelativeRotation), Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeRotationControlFromQuat(FAnimGenControlObject& Object, const FQuat Rotation, const FQuat RelativeRotation, const FName Tag)
{
	const FQuat LocalRotation = RelativeRotation.Inverse() * Rotation;
	const FVector LocalAxisForward = LocalRotation.GetForwardVector();
	const FVector LocalAxisRight = LocalRotation.GetRightVector();

	return MakeContinuousControlFromArrayView(Object, {
		(float)LocalAxisForward.X,
		(float)LocalAxisForward.Y,
		(float)LocalAxisForward.Z,
		(float)LocalAxisRight.X,
		(float)LocalAxisRight.Y,
		(float)LocalAxisRight.Z,
		}, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeScaleControl(FAnimGenControlObject& Object, const FVector Scale, const FVector RelativeScale, const FName Tag)
{
	const FVector LocalLogScale =
		UE::AnimDatabase::Math::VectorLogSafe(Scale) -
		UE::AnimDatabase::Math::VectorLogSafe(RelativeScale);

	return MakeContinuousControlFromArrayView(Object, {
		(float)LocalLogScale.X,
		(float)LocalLogScale.Y,
		(float)LocalLogScale.Z,
		}, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeTransformControl(FAnimGenControlObject& Object, const FTransform Transform, const FTransform& RelativeTransform, const FName Tag)
{
	const FTransform LocalTransform = Transform * RelativeTransform.Inverse();

	return MakeStructControlFromArrayViews(Object,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
		{
			MakeLocationControl(Object, LocalTransform.GetLocation(), FTransform::Identity),
			MakeRotationControlFromQuat(Object, LocalTransform.GetRotation(), FQuat::Identity),
			MakeScaleControl(Object, LocalTransform.GetScale3D(), FVector::OneVector)
		},
		Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeAngleControl(FAnimGenControlObject& Object, const float Angle, const float RelativeAngle, const FName Tag)
{
	const float LocalAngle = FMath::FindDeltaAngleDegrees(RelativeAngle, Angle);
	const float EncodedX = FMath::Sin(FMath::DegreesToRadians(LocalAngle));
	const float EncodedY = FMath::Cos(FMath::DegreesToRadians(LocalAngle));

	return MakeContinuousControlFromArrayView(Object, { EncodedX, EncodedY }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeAngleControlRadians(FAnimGenControlObject& Object, const float Angle, const float RelativeAngle, const FName Tag)
{
	return MakeAngleControl(Object, FMath::RadiansToDegrees(Angle), FMath::RadiansToDegrees(RelativeAngle), Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLinearVelocityControl(FAnimGenControlObject& Object, const FVector LinearVelocity, const FTransform& RelativeTransform, const FName Tag)
{
	const FVector LocalLinearVelocity = RelativeTransform.InverseTransformVectorNoScale(LinearVelocity);

	return MakeContinuousControlFromArrayView(Object, { (float)LocalLinearVelocity.X, (float)LocalLinearVelocity.Y, (float)LocalLinearVelocity.Z }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeAngularVelocityControl(FAnimGenControlObject& Object, const FVector AngularVelocity, const FTransform& RelativeTransform, const FName Tag)
{
	const FVector LocalAngularVelocity = RelativeTransform.InverseTransformVectorNoScale(AngularVelocity);

	return MakeContinuousControlFromArrayView(Object, { (float)LocalAngularVelocity.X, (float)LocalAngularVelocity.Y, (float)LocalAngularVelocity.Z }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeDirectionControl(FAnimGenControlObject& Object, const FVector Direction, const FTransform& RelativeTransform, const FName Tag)
{
	const FVector LocalDirection = RelativeTransform.InverseTransformVectorNoScale(Direction).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	return MakeContinuousControlFromArrayView(Object, {
		(float)LocalDirection.X,
		(float)LocalDirection.Y,
		(float)LocalDirection.Z,
		}, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeTimeControl(FAnimGenControlObject& Object, const float Time, const float RelativeTime, const FName Tag)
{
	return MakeContinuousControlFromArrayView(Object, { Time - RelativeTime }, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationsStaticArrayControl(FAnimGenControlObject& Object, const TArray<FVector>& Locations, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeLocationsStaticArrayControlFromArrayView(Object, Locations, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationsStaticArrayControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const FTransform& RelativeTransform, const FName Tag)
{
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> LocationElements;
	LocationElements.Reserve(Locations.Num());
	for (const FVector& Location : Locations)
	{
		LocationElements.Emplace(MakeLocationControl(Object, Location, RelativeTransform));
	}

	return MakeStaticArrayControlFromArrayView(Object, LocationElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLinearVelocitiesStaticArrayControl(FAnimGenControlObject& Object, const TArray<FVector>& LinearVelocities, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeLinearVelocitiesStaticArrayControlFromArrayView(Object, LinearVelocities, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLinearVelocitiesStaticArrayControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const FTransform& RelativeTransform, const FName Tag)
{
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> LinearVelocityElements;
	LinearVelocityElements.Reserve(Locations.Num());
	for (const FVector& Location : Locations)
	{
		LinearVelocityElements.Emplace(MakeLinearVelocityControl(Object, Location, RelativeTransform));
	}

	return MakeStaticArrayControlFromArrayView(Object, LinearVelocityElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeDirectionsStaticArrayControl(FAnimGenControlObject& Object, const TArray<FVector>& Directions, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeDirectionsStaticArrayControlFromArrayView(Object, Directions, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeDirectionsStaticArrayControlFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform, const FName Tag)
{
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> DirectionElements;
	DirectionElements.Reserve(Directions.Num());
	for (const FVector& Direction : Directions)
	{
		DirectionElements.Emplace(MakeDirectionControl(Object, Direction, RelativeTransform));
	}

	return MakeStaticArrayControlFromArrayView(Object, DirectionElements, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakePoseControl(FAnimGenControlObject& Object, const TArray<FVector>& Locations, const TArray<FVector>& LinearVelocities, const FTransform& RelativeTransform, const FName Tag)
{
	return MakePoseControlFromArrayViews(Object, Locations, LinearVelocities, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakePoseControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const TArrayView<const FVector> LinearVelocities, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeStructControlFromArrayViews(Object,
		{
			TEXT("Locations"),
			TEXT("LinearVelocities")
		},
		{
			MakeLocationsStaticArrayControlFromArrayView(Object, Locations, RelativeTransform),
			MakeLinearVelocitiesStaticArrayControlFromArrayView(Object, LinearVelocities, RelativeTransform),
		},
		Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakePoseControlFromPoseState(FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const TArray<int32>& BoneIndices, const FName Tag)
{
	return MakePoseControlFromPoseStateArrayViews(Object, PoseState, BoneIndices, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakePoseControlFromPoseStateArrayViews(FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const TArrayView<const int32> BoneIndices, const FName Tag)
{
	const FTransform RelativeTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(PoseState);

	TArray<FVector, TInlineAllocator<16>> BoneLocations;
	TArray<FVector, TInlineAllocator<16>> BoneLinearVelocities;
	BoneLocations.Reserve(BoneIndices.Num());
	BoneLinearVelocities.Reserve(BoneIndices.Num());

	for (const int32 BoneIdx : BoneIndices)
	{
		BoneLocations.Emplace(UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(PoseState, BoneIdx));
		BoneLinearVelocities.Emplace(UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(PoseState, BoneIdx));
	}

	return MakePoseControlFromArrayViews(Object, BoneLocations, BoneLinearVelocities, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeTrajectoryControl(FAnimGenControlObject& Object, const TArray<FVector>& Locations, const TArray<FVector>& Directions, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeTrajectoryControlFromArrayViews(Object, Locations, Directions, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationTrajectoryControl(FAnimGenControlObject& Object, const TArray<FVector>& Locations, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeLocationTrajectoryControlFromArrayViews(Object, Locations, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeTrajectoryControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeStructControlFromArrayViews(Object,
		{
			TEXT("Locations"),
			TEXT("Directions")
		},
		{
			MakeLocationsStaticArrayControlFromArrayView(Object, Locations, RelativeTransform),
			MakeDirectionsStaticArrayControlFromArrayView(Object, Directions, RelativeTransform),
		},
		Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationTrajectoryControlFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const FTransform& RelativeTransform, const FName Tag)
{
	return MakeLocationsStaticArrayControlFromArrayView(Object, Locations, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeTrajectoryControlFromTransformTrajectory(FAnimGenControlObject& Object, const FTransformTrajectory& TransformTrajectory, const FTransform& RelativeTransform, const int32 SampleNum, const float FutureTime, const float PastTime, const FVector ForwardVector, const FName Tag)
{
	if (SampleNum < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimDatabase, Error, "MakeTrajectoryControlFromTransformTrajectory: Invalid Sample Num: {SampleNum}.", SampleNum);
		return FAnimGenControlObjectElement();
	}

	TArray<FVector, TInlineAllocator<16>> Locations;
	TArray<FVector, TInlineAllocator<16>> Directions;
	Locations.SetNumUninitialized(SampleNum);
	Directions.SetNumUninitialized(SampleNum);

	for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		const float Time = (FutureTime + PastTime) * ((float)(SampleIdx) / FMath::Max(SampleNum - 1, 1)) - PastTime;
		FTransformTrajectorySample Sample = TransformTrajectory.GetSampleAtTime(Time);
		Locations[SampleIdx] = Sample.Position;
		Directions[SampleIdx] = Sample.Facing.RotateVector(ForwardVector);
	}

	return MakeTrajectoryControlFromArrayViews(Object, Locations, Directions, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeLocationTrajectoryControlFromTransformTrajectory(FAnimGenControlObject& Object, const FTransformTrajectory& TransformTrajectory, const FTransform& RelativeTransform, const int32 SampleNum, const float FutureTime, const float PastTime, const FName Tag)
{
	if (SampleNum < 0)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimDatabase, Error, "MakeLocationTrajectoryControlFromTransformTrajectory: Invalid Sample Num: {SampleNum}.", SampleNum);
		return FAnimGenControlObjectElement();
	}

	TArray<FVector, TInlineAllocator<16>> Locations;
	Locations.SetNumUninitialized(SampleNum);

	for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		const float Time = (FutureTime + PastTime) * ((float)(SampleIdx) / FMath::Max(SampleNum - 1, 1)) - PastTime;
		FTransformTrajectorySample Sample = TransformTrajectory.GetSampleAtTime(Time);
		Locations[SampleIdx] = Sample.Position;
	}

	return MakeLocationTrajectoryControlFromArrayViews(Object, Locations, RelativeTransform, Tag);
}

FAnimGenControlObjectElement UAnimGenControls::MakeEventControl(FAnimGenControlObject& Object, const bool bTimeUntilEventKnown, const float TimeUntilEvent, const FName Tag)
{
	if (!bTimeUntilEventKnown)
	{
		return MakeContinuousControlFromArrayView(Object,
			{
				0.0f,
				-1.0f
			}, Tag);
	}
	else
	{
		const float Aprehension = 1.0f;
		const float AprehensionPadding = 0.1f;
		const float ClampedTimeUntilEvent = FMath::Clamp(TimeUntilEvent / (Aprehension + AprehensionPadding), -1.0f, 1.0f);

		return MakeContinuousControlFromArrayView(Object,
			{
				FMath::Sin(UE_PI * ClampedTimeUntilEvent),
				FMath::Cos(UE_PI * ClampedTimeUntilEvent)
			}, Tag);
	}
}

FAnimGenControlObjectElement UAnimGenControls::MakeEventControlFromPoseState(FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const int32 FrameAttributeIndex, const FName Tag)
{
	bool bTimeUntilEventKnown = false;
	float TimeUntilEvent = UE_MAX_FLT;
	UAnimDatabasePoseStateLibrary::PoseStateEventAttribute(bTimeUntilEventKnown, TimeUntilEvent, PoseState, FrameAttributeIndex);
	return MakeEventControl(Object, bTimeUntilEventKnown, TimeUntilEvent, Tag);
}


bool UAnimGenControls::GetNullControl(const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNullControl: Invalid Control Object.");
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNullControl: Invalid Control Object Element.");
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNullControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Null)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNullControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Null));
			return false;
		}

		return true;
	}
}

bool UAnimGenControls::GetContinuousControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{

	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetContinuousControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Continuous)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Continuous));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetContinuous(Element.ObjectElement).Values.Num();
		return true;
	}
}

bool UAnimGenControls::GetContinuousControl(TArray<float>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutValueNum = 0;
	if (!GetContinuousControlNum(OutValueNum, Object, Element, Tag))
	{
		OutValues.Empty();
		return false;
	}

	OutValues.SetNumUninitialized(OutValueNum);

	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutValues.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetContinuousControlToArrayView(TArrayView<float> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetContinuousControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Continuous)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Continuous));
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		const TArrayView<const float> Values = Object.ObservationObject->Object.GetContinuous(Element.ObjectElement).Values;

		if (Values.Num() != OutValues.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetContinuousControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' values but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Values.Num(), OutValues.Num());
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		UE::Learning::Array::Copy<1, float>(OutValues, Values);
		return true;
	}
}

bool UAnimGenControls::GetNamedExclusiveDiscreteControl(FName& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedExclusiveDiscreteControl: Invalid Control Object.");
		OutElement = NAME_None;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedExclusiveDiscreteControl: Invalid Control Object Element.");
			OutElement = NAME_None;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNamedExclusiveDiscreteControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteExclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedExclusiveDiscreteControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteExclusive));
			OutElement = NAME_None;
			return false;
		}

		OutElement = Object.ObservationObject->Object.GetNamedDiscreteExclusive(Element.ObjectElement).ElementName;
		return true;
	}
}

bool UAnimGenControls::GetNamedInclusiveDiscreteControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNamedInclusiveDiscreteControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteInclusive));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames.Num();
		return true;
	}
}

bool UAnimGenControls::GetNamedInclusiveDiscreteControl(TArray<FName>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutValueNum = 0;
	if (!GetNamedInclusiveDiscreteControlNum(OutValueNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutValueNum);

	if (!GetNamedInclusiveDiscreteControlToArrayView(OutElements, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetNamedInclusiveDiscreteControlToArrayView(TArrayView<FName> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNamedInclusiveDiscreteControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteInclusive));
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			return false;
		}

		const TArrayView<const FName> Elements = Object.ObservationObject->Object.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames;

		if (Elements.Num() != OutElements.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedInclusiveDiscreteControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' values but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Elements.Num(), OutElements.Num());
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			return false;
		}

		UE::Learning::Array::Copy<1, FName>(OutElements, Elements);
		return true;
	}
}

bool UAnimGenControls::GetExclusiveDiscreteControl(int32& OutIndex, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveDiscreteControl: Invalid Control Object.");
		OutIndex = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveDiscreteControl: Invalid Control Object Element.");
			OutIndex = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetExclusiveDiscreteControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteExclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveDiscreteControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteExclusive));
			OutIndex = 0;
			return false;
		}

		OutIndex = Object.ObservationObject->Object.GetDiscreteExclusive(Element.ObjectElement).DiscreteIndex;
		return true;
	}
}

bool UAnimGenControls::GetInclusiveDiscreteControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetInclusiveDiscreteControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteInclusive));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices.Num();
		return true;
	}
}

bool UAnimGenControls::GetInclusiveDiscreteControl(TArray<int32>& OutIndices, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutValueNum = 0;
	if (!GetInclusiveDiscreteControlNum(OutValueNum, Object, Element, Tag))
	{
		OutIndices.Empty();
		return false;
	}

	OutIndices.SetNumUninitialized(OutValueNum);

	if (!GetInclusiveDiscreteControlToArrayView(OutIndices, Object, Element, Tag))
	{
		OutIndices.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetInclusiveDiscreteControlToArrayView(TArrayView<int32> OutIndices, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetInclusiveDiscreteControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteInclusive));
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			return false;
		}

		const TArrayView<const int32> Indices = Object.ObservationObject->Object.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices;

		if (Indices.Num() != OutIndices.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveDiscreteControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' values but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Indices.Num(), OutIndices.Num());
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			return false;
		}

		UE::Learning::Array::Copy<1, int32>(OutIndices, Indices);
		return true;
	}
}

bool UAnimGenControls::GetCountControl(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag)
{
	float FloatNum = 0.0f;
	if (!GetContinuousControlToArrayView(MakeArrayView(&FloatNum, 1), Object, Element, Tag))
	{
		OutNum = -1;
		return false;
	}

	OutNum = FMath::RoundToInt(FloatNum * MaxNum);
	return true;
}

bool UAnimGenControls::GetStructControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStructControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
			OutNum = 0;
			return false;
		}

		const UE::Learning::Observation::FObjectAndParameters Parameters = Object.ObservationObject->Object.GetAnd(Element.ObjectElement);

		OutNum = Parameters.Elements.Num();
		return true;
	}
}

bool UAnimGenControls::GetStructControl(TMap<FName, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetStructControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetStructControlToArrayViews(SubElementNames, SubElements, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool UAnimGenControls::GetStructControlToArrays(TArray<FName>& OutElementNames, TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetStructControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStructControlToArrayViews(OutElementNames, OutElements, Object, Element, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetStructControlToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlToArrayViews: Invalid Control Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlToArrayViews: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStructControlToArrayViews: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlToArrayViews: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		const UE::Learning::Observation::FObjectAndParameters Parameters = Object.ObservationObject->Object.GetAnd(Element.ObjectElement);

		if (Parameters.Elements.Num() == 0)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStructControlToArrayViews: Getting zero-sized Struct Control.");
		}

		if (Parameters.Elements.Num() != OutElements.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlToArrayViews: Control '{ControlName}' size does not match. Control is '{ControlSize}' elements but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Parameters.Elements.Num(), OutElements.Num());
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			if (!Object.ObservationObject->Object.IsValid(Parameters.Elements[ElementIdx]))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlToArrayViews: Invalid Control Object.");
				UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
				UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
				return false;
			}

			OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
			OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
		}

		return true;
	}
}

bool UAnimGenControls::GetStructControlElement(FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlElement: Invalid Control Object.");
		OutElement = FAnimGenControlObjectElement();
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlElement: Invalid Control Object Element.");
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStructControlElement: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlElement: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		const UE::Learning::Observation::FObjectAndParameters Parameters = Object.ObservationObject->Object.GetAnd(Element.ObjectElement);

		if (Parameters.Elements.Num() == 0)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStructControlElement: Getting zero-sized Struct Control.");
		}

		const int32 ElementIdx = Parameters.ElementNames.Find(ElementName);

		if (ElementIdx == INDEX_NONE)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStructControlElement: Element '{ElementName}' not found.", *ElementName.ToString());
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		OutElement = { Parameters.Elements[ElementIdx] };
		return true;
	}
}

bool UAnimGenControls::GetExclusiveUnionControl(FName& OutElementName, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveUnionControl: Invalid Control Object.");
		OutElementName = NAME_None;
		OutElement = FAnimGenControlObjectElement();
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveUnionControl: Invalid Control Object Element.");
			OutElementName = NAME_None;
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetExclusiveUnionControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrExclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetExclusiveUnionControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrExclusive));
			OutElementName = NAME_None;
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		const UE::Learning::Observation::FObjectOrExclusiveParameters Parameters = Object.ObservationObject->Object.GetOrExclusive(Element.ObjectElement);
		OutElementName = Parameters.ElementName;
		OutElement = { Parameters.Element };
		return true;
	}
}

bool UAnimGenControls::GetInclusiveUnionControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetInclusiveUnionControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrInclusive));
			OutNum = 0;
			return false;
		}

		const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object.ObservationObject->Object.GetOrInclusive(Element.ObjectElement);

		OutNum = Parameters.Elements.Num();
		return true;
	}
}

bool UAnimGenControls::GetInclusiveUnionControl(TMap<FName, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetInclusiveUnionControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionControlToArrayViews(SubElementNames, SubElements, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool UAnimGenControls::GetInclusiveUnionControlToArrays(TArray<FName>& OutElementNames, TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetInclusiveUnionControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionControlToArrayViews(OutElementNames, OutElements, Object, Element, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetInclusiveUnionControlToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlToArrayViews: Invalid Control Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlToArrayViews: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetInclusiveUnionControlToArrayViews: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlToArrayViews: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrInclusive));
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object.ObservationObject->Object.GetOrInclusive(Element.ObjectElement);

		if (Parameters.Elements.Num() != OutElements.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlToArrayViews: Control '{ControlName}' size does not match. Control is '{ControlSize}' elements but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Parameters.Elements.Num(), OutElements.Num());
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			if (!Object.ObservationObject->Object.IsValid(Parameters.Elements[ElementIdx]))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetInclusiveUnionControlToArrayViews: Invalid Control Object Element.");
				UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
				UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
				return false;
			}

			OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
			OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
		}

		return true;
	}
}

bool UAnimGenControls::FindInclusiveUnionControlElement(FAnimGenControlObjectElement& OutElement, bool& bFound, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "FindInclusiveUnionControlElement: Invalid Control Object.");
		bFound = false;
		OutElement = FAnimGenControlObjectElement();
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "FindInclusiveUnionControlElement: Invalid Control Object Element.");
			bFound = false;
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "FindInclusiveUnionControlElement: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrInclusive)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "FindInclusiveUnionControlElement: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrInclusive));
			bFound = false;
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object.ObservationObject->Object.GetOrInclusive(Element.ObjectElement);

		const int32 ElementIndex = Parameters.ElementNames.Find(ElementName);
		if (ElementIndex == INDEX_NONE)
		{
			bFound = false;
			OutElement = FAnimGenControlObjectElement();
			return true;
		}

		bFound = true;
		OutElement.ObjectElement = Parameters.Elements[ElementIndex];
		return true;
	}
}

bool UAnimGenControls::SwitchOnHasInclusiveUnionControlElement(FAnimGenControlObjectElement& OutElement, bool& bFound, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag)
{
	return FindInclusiveUnionControlElement(OutElement, bFound, Object, Element, ElementName, Tag);
}

bool UAnimGenControls::GetStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStaticArrayControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Array)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Array));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetArray(Element.ObjectElement).Elements.Num();
		return true;
	}
}

bool UAnimGenControls::GetStaticArrayControl(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetStaticArrayControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStaticArrayControlToArrayView(OutElements, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetStaticArrayControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStaticArrayControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Array)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Array));
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		const UE::Learning::Observation::FObjectArrayParameters Parameters = Object.ObservationObject->Object.GetArray(Element.ObjectElement);

		if (Parameters.Elements.Num() == 0)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetStaticArrayControlToArrayView: Getting zero-sized Static Array Control.");
		}

		if (Parameters.Elements.Num() != OutElements.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' elements but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Parameters.Elements.Num(), OutElements.Num());
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			if (!Object.ObservationObject->Object.IsValid(Parameters.Elements[ElementIdx]))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetStaticArrayControlToArrayView: Invalid Control Object Element.");
				UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
				return false;
			}

			OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
		}

		return true;
	}
}

bool UAnimGenControls::GetSetControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetSetControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetSet(Element.ObjectElement).Elements.Num();
		return true;
	}
}

bool UAnimGenControls::GetSetControl(TSet<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControl: Invalid Control Object.");
		OutElements.Empty();
		return false;
	}

	int32 OutElementNum = 0;
	if (!GetSetControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControl: Invalid Control Object Element.");
			OutElements.Empty();
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetSetControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
			OutElements.Empty();
			return false;
		}

		const UE::Learning::Observation::FObjectSetParameters Parameters = Object.ObservationObject->Object.GetSet(Element.ObjectElement);

		OutElements.Empty(Parameters.Elements.Num());
		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			if (!Object.ObservationObject->Object.IsValid(Parameters.Elements[ElementIdx]))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControl: Invalid Control Object Element.");
				OutElements.Empty();
				return false;
			}

			OutElements.Add({ Parameters.Elements[ElementIdx] });
		}

		return true;
	}
}

bool UAnimGenControls::GetSetControlToArray(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetSetControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetSetControlToArrayView(OutElements, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetSetControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetSetControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		const UE::Learning::Observation::FObjectSetParameters Parameters = Object.ObservationObject->Object.GetSet(Element.ObjectElement);

		if (Parameters.Elements.Num() != OutElements.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' elements but asked for '{AskSize}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Parameters.Elements.Num(), OutElements.Num());
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			if (!Object.ObservationObject->Object.IsValid(Parameters.Elements[ElementIdx]))
			{
				UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSetControlToArrayView: Invalid Control Object Element.");
				UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
				return false;
			}

			OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
		}

		return true;
	}
}

bool UAnimGenControls::GetNamedSparseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNamedSparseControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedSparse)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedSparse));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetNamedSparse(Element.ObjectElement).ElementNames.Num();
		return true;
	}
}

bool UAnimGenControls::GetNamedSparseControl(TArray<FName>& OutElements, TArray<float>& OutElementValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetNamedSparseControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		OutElementValues.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);
	OutElementValues.SetNumUninitialized(OutElementNum);

	if (!GetNamedSparseControlToArrayView(OutElements, OutElementValues, Object, Element, Tag))
	{
		OutElements.Empty();
		OutElementValues.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetNamedSparseControlToArrayView(TArrayView<FName> OutElements, TArrayView<float> OutElementValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
		UE::Learning::Array::Zero<1, float>(OutElementValues);
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			UE::Learning::Array::Zero<1, float>(OutElementValues);
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetNamedSparseControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedSparse)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedSparse));
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			UE::Learning::Array::Zero<1, float>(OutElementValues);
			return false;
		}

		const TArrayView<const FName> Elements = Object.ObservationObject->Object.GetNamedSparse(Element.ObjectElement).ElementNames;
		const TArrayView<const float> Values = Object.ObservationObject->Object.GetNamedSparse(Element.ObjectElement).Values;

		if (Elements.Num() != OutElements.Num() ||
			Values.Num() != OutElementValues.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetNamedSparseControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' values but asked for '{AskSize}' and '{AskSize2}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Elements.Num(), OutElements.Num(), OutElementValues.Num());
			UE::Learning::Array::Set<1, FName>(OutElements, NAME_None);
			UE::Learning::Array::Zero<1, float>(OutElementValues);
			return false;
		}

		UE::Learning::Array::Copy<1, FName>(OutElements, Elements);
		UE::Learning::Array::Copy<1, float>(OutElementValues, Values);
		return true;
	}
}

bool UAnimGenControls::GetSparseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlNum: Invalid Control Object.");
		OutNum = 0;
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlNum: Invalid Control Object Element.");
			OutNum = 0;
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetSparseControlNum: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Sparse)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlNum: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Sparse));
			OutNum = 0;
			return false;
		}

		OutNum = Object.ObservationObject->Object.GetSparse(Element.ObjectElement).DiscreteIndices.Num();
		return true;
	}
}

bool UAnimGenControls::GetSparseControl(TArray<int32>& OutIndices, TArray<float>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutIndexNum = 0;
	if (!GetSparseControlNum(OutIndexNum, Object, Element, Tag))
	{
		OutIndices.Empty();
		OutValues.Empty();
		return false;
	}

	OutIndices.SetNumUninitialized(OutIndexNum);
	OutValues.SetNumUninitialized(OutIndexNum);

	if (!GetSparseControlToArrayView(OutIndices, OutValues, Object, Element, Tag))
	{
		OutIndices.Empty();
		OutValues.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetSparseControlToArrayView(TArrayView<int32> OutIndices, TArrayView<float> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlToArrayView: Invalid Control Object.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlToArrayView: Invalid Control Object Element.");
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetSparseControlToArrayView: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Sparse)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlToArrayView: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Sparse));
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		const TArrayView<const int32> Indices = Object.ObservationObject->Object.GetSparse(Element.ObjectElement).DiscreteIndices;
		const TArrayView<const float> Values = Object.ObservationObject->Object.GetSparse(Element.ObjectElement).Values;

		if (Indices.Num() != OutIndices.Num() ||
			Values.Num() != OutValues.Num())
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetSparseControlToArrayView: Control '{ControlName}' size does not match. Control is '{ControlSize}' values but asked for '{AskSize}' and '{AskSize2}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				Indices.Num(), OutIndices.Num(), OutValues.Num());
			UE::Learning::Array::Zero<1, int32>(OutIndices);
			UE::Learning::Array::Zero<1, float>(OutValues);
			return false;
		}

		UE::Learning::Array::Copy<1, int32>(OutIndices, Indices);
		UE::Learning::Array::Copy<1, float>(OutValues, Values);
		return true;
	}
}

bool UAnimGenControls::GetPairControl(FAnimGenControlObjectElement& OutKey, FAnimGenControlObjectElement& OutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	TStaticArray<FName, 2> OutElementNames;
	TStaticArray<FAnimGenControlObjectElement, 2> OutElements;
	if (!GetStructControlToArrayViews(OutElementNames, OutElements, Object, Element, Tag))
	{
		OutKey = FAnimGenControlObjectElement();
		OutValue = FAnimGenControlObjectElement();
		return false;
	}

	const int32 KeyIndex = MakeArrayView(OutElementNames).Find(TEXT("Key"));
	const int32 ValueIndex = MakeArrayView(OutElementNames).Find(TEXT("Value"));

	if (KeyIndex == INDEX_NONE || ValueIndex == INDEX_NONE)
	{
		OutKey = FAnimGenControlObjectElement();
		OutValue = FAnimGenControlObjectElement();
		return false;
	}

	OutKey = OutElements[KeyIndex];
	OutValue = OutElements[ValueIndex];
	return true;
}

bool UAnimGenControls::GetArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetSetControlNum(OutNum, Object, Element, Tag);
}

bool UAnimGenControls::GetArrayControl(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetArrayControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetArrayControlToArrayView(OutElements, Object, Element, MaxNum, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetArrayControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag)
{
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutElements.Num());
	if (!GetSetControlToArrayView(Pairs, Object, Element, Tag))
	{
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
		return false;
	}

	for (int32 PairIdx = 0; PairIdx < Pairs.Num(); PairIdx++)
	{
		FAnimGenControlObjectElement Key, Value;
		if (!GetPairControl(Key, Value, Object, Pairs[PairIdx]))
		{
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}

		int32 Index = INDEX_NONE;
		if (!GetCountControl(Index, Object, Key, MaxNum))
		{
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutElements, FAnimGenControlObjectElement());
			return false;
		}
		Index = FMath::Clamp(Index, 0, OutElements.Num() - 1);

		OutElements[Index] = Value;
	}

	return true;
}

bool UAnimGenControls::GetMapControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetSetControlNum(OutNum, Object, Element, Tag);
}

bool UAnimGenControls::GetMapControl(TMap<FAnimGenControlObjectElement, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetMapControlNum(OutElementNum, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutElementNum);
	if (!GetSetControlToArrayView(Pairs, Object, Element, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 PairIdx = 0; PairIdx < OutElementNum; PairIdx++)
	{
		FAnimGenControlObjectElement Key, Value;
		if (!GetPairControl(Key, Value, Object, Pairs[PairIdx]))
		{
			OutElements.Empty();
			return false;
		}

		OutElements.Add(Key, Value);
	}

	return true;
}

bool UAnimGenControls::GetMapControlToArrays(TArray<FAnimGenControlObjectElement>& OutKeys, TArray<FAnimGenControlObjectElement>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	int32 OutElementNum = 0;
	if (!GetMapControlNum(OutElementNum, Object, Element, Tag))
	{
		OutKeys.Empty();
		OutValues.Empty();
		return false;
	}

	OutKeys.SetNumUninitialized(OutElementNum);
	OutValues.SetNumUninitialized(OutElementNum);
	if (!GetMapControlToArrayViews(OutKeys, OutValues, Object, Element, Tag))
	{
		OutKeys.Empty();
		OutValues.Empty();
		return false;
	}

	return true;
}

bool UAnimGenControls::GetMapControlToArrayViews(TArrayView<FAnimGenControlObjectElement> OutKeys, TArrayView<FAnimGenControlObjectElement> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutKeys.Num());
	if (!GetSetControlToArrayView(Pairs, Object, Element, Tag))
	{
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutKeys, FAnimGenControlObjectElement());
		UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutValues, FAnimGenControlObjectElement());
		return false;
	}

	for (int32 PairIdx = 0; PairIdx < Pairs.Num(); PairIdx++)
	{
		FAnimGenControlObjectElement Key, Value;
		if (!GetPairControl(Key, Value, Object, Pairs[PairIdx]))
		{
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutKeys, FAnimGenControlObjectElement());
			UE::Learning::Array::Set<1, FAnimGenControlObjectElement>(OutValues, FAnimGenControlObjectElement());
			return false;
		}

		OutKeys[PairIdx] = Key;
		OutValues[PairIdx] = Value;
	}

	return true;
}

bool UAnimGenControls::GetEnumControl(uint8& OutEnumValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const UEnum* Enum, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetEnumControl: Enum is nullptr.");
		OutEnumValue = 0;
		return false;
	}

	FName EnumValueNum = NAME_None;
	if (!GetNamedExclusiveDiscreteControl(EnumValueNum, Object, Element, Tag))
	{
		OutEnumValue = 0;
		return false;
	}

	OutEnumValue = Enum->GetValueByName(EnumValueNum);
	return true;
}

bool UAnimGenControls::GetBitmaskControl(int32& OutBitmaskValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const UEnum* Enum, const FName Tag)
{
	if (!Enum)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetBitmaskControl: Enum is nullptr.");
		OutBitmaskValue = 0;
		return false;
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetBitmaskControl: Too many values in Enum to use as Bitmask ({EnumSize}).", Enum->NumEnums() - 1);
		OutBitmaskValue = 0;
		return false;
	}

	int32 EnumValueNum;
	if (!GetNamedInclusiveDiscreteControlNum(EnumValueNum, Object, Element, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	TArray<FName, TInlineAllocator<32>> Names;
	Names.SetNumUninitialized(EnumValueNum);
	if (!GetNamedInclusiveDiscreteControlToArrayView(Names, Object, Element, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	OutBitmaskValue = 0;
	for (int32 ValueIdx = 0; ValueIdx < EnumValueNum; ValueIdx++)
	{
		if (Names.Contains(Enum->GetNameByIndex(ValueIdx)))
		{
			OutBitmaskValue |= (1 << ValueIdx);
		}
	}

	return true;
}


bool UAnimGenControls::GetOptionalControl(EAnimGenOptionalControl& OutOption, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	FName OutName = NAME_None;
	if (!GetExclusiveUnionControl(OutName, OutElement, Object, Element, Tag))
	{
		OutOption = EAnimGenOptionalControl::Null;
		return false;
	}

	OutOption = OutName == TEXT("Null") ? EAnimGenOptionalControl::Null : EAnimGenOptionalControl::Valid;
	return true;
}

bool UAnimGenControls::GetEitherControl(EAnimGenEitherControl& OutEither, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	FName OutName = NAME_None;
	if (!GetExclusiveUnionControl(OutName, OutElement, Object, Element, Tag))
	{
		OutEither = EAnimGenEitherControl::A;
		return false;
	}

	OutEither = OutName == TEXT("A") ? EAnimGenEitherControl::A : EAnimGenEitherControl::B;
	return true;
}

bool UAnimGenControls::GetEncodingControl(FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!Object.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetEncodingControl: Invalid Control Object.");
		OutElement = FAnimGenControlObjectElement();
		return false;
	}

	{
		UE::AnimGen::FSpinLockScope ObjectLock(Object.ObservationObject->Lock);

		if (!Object.ObservationObject->Object.IsValid(Element.ObjectElement))
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetEncodingControl: Invalid Control Object Element.");
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		if (Object.ObservationObject->Object.GetTag(Element.ObjectElement) != Tag)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Warning, "GetEncodingControl: Control tag does not match. Control is '{ControlTag}' but asked for '{AskTag}'.", *Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
		}

		if (Object.ObservationObject->Object.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Encoding)
		{
			UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "GetEncodingControl: Control '{ControlName}' type does not match. Control is '{ControlType}' but asked for '{AskType}'.",
				*Object.ObservationObject->Object.GetTag(Element.ObjectElement).ToString(),
				UE::AnimGen::Control::Private::GetObservationTypeString(Object.ObservationObject->Object.GetType(Element.ObjectElement)),
				UE::AnimGen::Control::Private::GetObservationTypeString(UE::Learning::Observation::EType::Encoding));
			OutElement = FAnimGenControlObjectElement();
			return false;
		}

		OutElement = { Object.ObservationObject->Object.GetEncoding(Element.ObjectElement).Element };

		return true;
	}
}

bool UAnimGenControls::GetBoolControl(bool& bOutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	FName OutElement = NAME_None;
	if (!GetNamedExclusiveDiscreteControl(OutElement, Object, Element, Tag))
	{
		bOutValue = false;
		return false;
	}

	bOutValue = OutElement == TEXT("true");
	return true;
}

bool UAnimGenControls::GetFloatControl(float& OutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	if (!GetContinuousControlToArrayView(MakeArrayView(&OutValue, 1), Object, Element, Tag))
	{
		OutValue = 0.0f;
		return false;
	}

	return true;
}

bool UAnimGenControls::GetLocationControl(FVector& OutLocation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	TStaticArray<float, 3> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutLocation = FVector::ZeroVector;
		return false;
	}

	OutLocation = RelativeTransform.TransformPosition(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool UAnimGenControls::GetRotationControl(FRotator& OutRotation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FRotator RelativeRotation, const FName Tag)
{
	FQuat OutRotationQuat;
	if (!GetRotationControlAsQuat(OutRotationQuat, Object, Element, FQuat::MakeFromRotator(RelativeRotation), Tag))
	{
		OutRotation = FRotator::ZeroRotator;
		return false;
	}

	OutRotation = OutRotationQuat.Rotator();
	return true;
}

bool UAnimGenControls::GetRotationControlAsQuat(FQuat& OutRotation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FQuat RelativeRotation, const FName Tag)
{
	TStaticArray<float, 6> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutRotation = FQuat::Identity;
		return false;
	}

	const FVector LocalAxisForward = FVector(OutValues[0], OutValues[1], OutValues[2]);
	const FVector LocalAxisRight = FVector(OutValues[3], OutValues[4], OutValues[5]);
	const FVector AxisUp = LocalAxisForward.Cross(LocalAxisRight).GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
	const FVector AxisRight = AxisUp.Cross(LocalAxisForward).GetSafeNormal(UE_SMALL_NUMBER, FVector::RightVector);
	const FVector AxisForward = LocalAxisForward.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	FMatrix RotationMatrix = FMatrix::Identity;
	RotationMatrix.SetAxis(0, AxisForward);
	RotationMatrix.SetAxis(1, AxisRight);
	RotationMatrix.SetAxis(2, AxisUp);

	OutRotation = RelativeRotation * RotationMatrix.ToQuat();
	return true;
}

bool UAnimGenControls::GetScaleControl(FVector& OutScale, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FVector RelativeScale, const FName Tag)
{
	TStaticArray<float, 3> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutScale = FVector::OneVector;
		return false;
	}

	OutScale = RelativeScale * UE::AnimDatabase::Math::VectorExpSafe(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool UAnimGenControls::GetTransformControl(FTransform& OutTransform, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	TStaticArray<FName, 3> OutElementNames;
	TStaticArray<FAnimGenControlObjectElement, 3> OutElements;
	if (!GetStructControlToArrayViews(OutElementNames, OutElements, Object, Element, Tag))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 LocationElement = MakeArrayView(OutElementNames).Find(TEXT("Location"));
	FVector OutLocation;
	if (LocationElement == INDEX_NONE || !GetLocationControl(OutLocation, Object, OutElements[LocationElement], RelativeTransform))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 RotationElement = MakeArrayView(OutElementNames).Find(TEXT("Rotation"));
	FQuat OutRotation;
	if (RotationElement == INDEX_NONE || !GetRotationControlAsQuat(OutRotation, Object, OutElements[RotationElement], RelativeTransform.GetRotation()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 ScaleElement = MakeArrayView(OutElementNames).Find(TEXT("Scale"));
	FVector OutScale;
	if (ScaleElement == INDEX_NONE || !GetScaleControl(OutScale, Object, OutElements[ScaleElement], RelativeTransform.GetScale3D()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	OutTransform = FTransform(OutRotation, OutLocation, OutScale);
	return true;
}

bool UAnimGenControls::GetAngleControlRadians(float& OutAngle, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeAngle, const FName Tag)
{
	TStaticArray<float, 2> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutAngle = 0.0f;
		return false;
	}

	OutAngle = RelativeAngle + FMath::Atan2(OutValues[0], OutValues[1]);
	return true;
}


bool UAnimGenControls::GetAngleControl(float& OutAngle, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeAngle, const FName Tag)
{
	if (!GetAngleControlRadians(OutAngle, Object, Element, FMath::DegreesToRadians(RelativeAngle), Tag))
	{
		return false;
	}

	OutAngle = FMath::RadiansToDegrees(OutAngle);
	return true;
}

bool UAnimGenControls::GetLinearVelocityControl(FVector& OutLinearVelocity, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	TStaticArray<float, 3> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutLinearVelocity = FVector::ZeroVector;
		return false;
	}

	OutLinearVelocity = RelativeTransform.TransformVectorNoScale(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool UAnimGenControls::GetAngularVelocityControl(FVector& OutAngularVelocity, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	TStaticArray<float, 3> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutAngularVelocity = FVector::ZeroVector;
		return false;
	}

	OutAngularVelocity = RelativeTransform.TransformVectorNoScale(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool UAnimGenControls::GetDirectionControl(FVector& OutDirection, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	TStaticArray<float, 3> OutValues;
	if (!GetContinuousControlToArrayView(OutValues, Object, Element, Tag))
	{
		OutDirection = FVector::ForwardVector;
		return false;
	}

	OutDirection = RelativeTransform.TransformVectorNoScale(FVector(OutValues[0], OutValues[1], OutValues[2]).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector));
	return true;
}

bool UAnimGenControls::GetTimeControl(float& OutTime, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeTime, const FName Tag)
{
	if (!GetContinuousControlToArrayView(MakeArrayView(&OutTime, 1), Object, Element, Tag))
	{
		OutTime = 0.0f;
		return false;
	}

	OutTime = OutTime + RelativeTime;
	return true;
}

bool UAnimGenControls::GetLocationsStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetStaticArrayControlNum(OutNum, Object, Element, Tag);
}

bool UAnimGenControls::GetLocationsStaticArrayControl(TArray<FVector>& OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	int32 Num = 0;
	if (!GetLocationsStaticArrayControlNum(Num, Object, Element, Tag))
	{
		OutLocations.Empty();
		return false;
	}

	OutLocations.SetNumUninitialized(Num);
	return GetLocationsStaticArrayControlToArrayView(OutLocations, Object, Element, RelativeTransform, Tag);
}

bool UAnimGenControls::GetLocationsStaticArrayControlToArrayView(TArrayView<FVector> OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	const int32 ElementNum = OutLocations.Num();

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Elements;
	Elements.SetNumUninitialized(ElementNum);
	if (!GetStaticArrayControlToArrayView(Elements, Object, Element, Tag))
	{
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!GetLocationControl(OutLocations[ElementIdx], Object, Elements[ElementIdx], RelativeTransform))
		{
			return false;
		}
	}

	return true;
}

bool UAnimGenControls::GetLinearVelocitiesStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetStaticArrayControlNum(OutNum, Object, Element, Tag);
}

bool UAnimGenControls::GetLinearVelocitiesStaticArrayControl(TArray<FVector>& OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	int32 Num = 0;
	if (!GetLinearVelocitiesStaticArrayControlNum(Num, Object, Element, Tag))
	{
		OutLinearVelocities.Empty();
		return false;
	}

	OutLinearVelocities.SetNumUninitialized(Num);
	return GetLinearVelocitiesStaticArrayControlToArrayView(OutLinearVelocities, Object, Element, RelativeTransform, Tag);
}

bool UAnimGenControls::GetLinearVelocitiesStaticArrayControlToArrayView(TArrayView<FVector> OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	const int32 ElementNum = OutLinearVelocities.Num();

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Elements;
	Elements.SetNumUninitialized(ElementNum);
	if (!GetStaticArrayControlToArrayView(Elements, Object, Element, Tag))
	{
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!GetLinearVelocityControl(OutLinearVelocities[ElementIdx], Object, Elements[ElementIdx], RelativeTransform))
		{
			return false;
		}
	}

	return true;
}

bool UAnimGenControls::GetDirectionsStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetStaticArrayControlNum(OutNum, Object, Element, Tag);
}

bool UAnimGenControls::GetDirectionsStaticArrayControl(TArray<FVector>& OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	int32 Num = 0;
	if (!GetDirectionsStaticArrayControlNum(Num, Object, Element, Tag))
	{
		OutDirections.Empty();
		return false;
	}

	OutDirections.SetNumUninitialized(Num);
	return GetDirectionsStaticArrayControlToArrayView(OutDirections, Object, Element, RelativeTransform, Tag);
}

bool UAnimGenControls::GetDirectionsStaticArrayControlToArrayView(TArrayView<FVector> OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	const int32 ElementNum = OutDirections.Num();

	TArray<FAnimGenControlObjectElement, TInlineAllocator<16>> Elements;
	Elements.SetNumUninitialized(ElementNum);
	if (!GetStaticArrayControlToArrayView(Elements, Object, Element, Tag))
	{
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!GetDirectionControl(OutDirections[ElementIdx], Object, Elements[ElementIdx], RelativeTransform))
		{
			return false;
		}
	}

	return true;
}

bool UAnimGenControls::GetPoseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControlNum(OutNum, Object, LocationsElement))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetPoseControl(TArray<FVector>& OutLocations, TArray<FVector>& OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControl(OutLocations, Object, LocationsElement, RelativeTransform))
	{
		return false;
	}

	FAnimGenControlObjectElement LinearVelocitiesElement;
	if (!GetStructControlElement(LinearVelocitiesElement, Object, Element, TEXT("LinearVelocities"), Tag))
	{
		return false;
	}

	if (!GetLinearVelocitiesStaticArrayControl(OutLinearVelocities, Object, LinearVelocitiesElement, RelativeTransform))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetPoseControlToArrayViews(TArrayView<FVector> OutLocations, TArrayView<FVector> OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControlToArrayView(OutLocations, Object, LocationsElement, RelativeTransform))
	{
		return false;
	}

	FAnimGenControlObjectElement LinearVelocitiesElement;
	if (!GetStructControlElement(LinearVelocitiesElement, Object, Element, TEXT("LinearVelocities"), Tag))
	{
		return false;
	}

	if (!GetLinearVelocitiesStaticArrayControlToArrayView(OutLinearVelocities, Object, LinearVelocitiesElement, RelativeTransform))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetTrajectoryControlNum(int32& OutLocationNum, int32& OutDirectionNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControlNum(OutLocationNum, Object, LocationsElement))
	{
		return false;
	}

	FAnimGenControlObjectElement DirectionsElement;
	if (!GetStructControlElement(DirectionsElement, Object, Element, TEXT("Directions"), Tag))
	{
		return false;
	}

	if (!GetDirectionsStaticArrayControlNum(OutDirectionNum, Object, DirectionsElement))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetTrajectoryControl(TArray<FVector>& OutLocations, TArray<FVector>& OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControl(OutLocations, Object, LocationsElement, RelativeTransform))
	{
		return false;
	}

	FAnimGenControlObjectElement DirectionsElement;
	if (!GetStructControlElement(DirectionsElement, Object, Element, TEXT("Directions"), Tag))
	{
		return false;
	}

	if (!GetDirectionsStaticArrayControl(OutDirections, Object, DirectionsElement, RelativeTransform))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetTrajectoryControlToArrayViews(TArrayView<FVector> OutLocations, TArrayView<FVector> OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	FAnimGenControlObjectElement LocationsElement;
	if (!GetStructControlElement(LocationsElement, Object, Element, TEXT("Locations"), Tag))
	{
		return false;
	}

	if (!GetLocationsStaticArrayControlToArrayView(OutLocations, Object, LocationsElement, RelativeTransform))
	{
		return false;
	}

	FAnimGenControlObjectElement DirectionsElement;
	if (!GetStructControlElement(DirectionsElement, Object, Element, TEXT("Directions"), Tag))
	{
		return false;
	}

	if (!GetDirectionsStaticArrayControlToArrayView(OutDirections, Object, DirectionsElement, RelativeTransform))
	{
		return false;
	}

	return true;
}

bool UAnimGenControls::GetLocationTrajectoryControlNum(int32& OutLocationNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	return GetLocationsStaticArrayControlNum(OutLocationNum, Object, Element, Tag);
}

bool UAnimGenControls::GetLocationTrajectoryControl(TArray<FVector>& OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	return GetLocationsStaticArrayControl(OutLocations, Object, Element, RelativeTransform, Tag);
}

bool UAnimGenControls::GetLocationTrajectoryControlToArrayViews(TArrayView<FVector> OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FName Tag)
{
	return GetLocationsStaticArrayControlToArrayView(OutLocations, Object, Element, RelativeTransform, Tag);
}

bool UAnimGenControls::GetEventControl(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag)
{
	TStaticArray<float, 2> Values;
	if (!GetContinuousControlToArrayView(MakeArrayView(Values), Object, Element, Tag))
	{
		bOutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	const float Aprehension = 1.0f;
	const float AprehensionPadding = 0.1f;
	const float TimeUntilEvent = (Aprehension + AprehensionPadding) * (FMath::Atan2(Values[0], Values[1]) / UE_PI);

	if (TimeUntilEvent < +Aprehension && TimeUntilEvent > -Aprehension)
	{
		bOutTimeUntilEventKnown = true;
		OutTimeUntilEvent = TimeUntilEvent;
	}
	else
	{
		bOutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
	}

	return true;
}

bool UAnimGenControls::FrameAttributeLocationDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& LocationFrameAttribute, const float Scale, const float Weight, const FName Tag)
{
	FVector Location;
	if (!GetLocationControl(Location, Object, Element, FTransform::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMatchingDistance(LocationFrameAttribute, Location, Scale, Weight);
	return true;
}

bool UAnimGenControls::FrameAttributeVelocityDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& VelocityFrameAttribute, const float Scale, const float Weight, const FName Tag)
{
	FVector LinearVelocity;
	if (!GetLinearVelocityControl(LinearVelocity, Object, Element, FTransform::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMatchingDistance(VelocityFrameAttribute, LinearVelocity, Scale, Weight);
	return true;
}

bool UAnimGenControls::FrameAttributeDirectionDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute, const float Weight, const FName Tag)
{
	FVector Direction;
	if (!GetDirectionControl(Direction, Object, Element, FTransform::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionMatchingDistance(DirectionFrameAttribute, Direction, Weight);
	return true;
}

bool UAnimGenControls::FrameAttributeRotationDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& RotationFrameAttribute, const float Weight, const FName Tag)
{
	FQuat Rotation;
	if (!GetRotationControlAsQuat(Rotation, Object, Element, FQuat::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMatchingDistance(RotationFrameAttribute, Rotation, Weight);
	return true;
}

bool UAnimGenControls::FrameAttributeEventDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& EventFrameAttribute, const float Weight, const FName Tag)
{
	bool bTimeUntilEventKnown;
	float TimeUntilEvent;
	if (!GetEventControl(bTimeUntilEventKnown, TimeUntilEvent, Object, Element, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeEventMatchingDistance(EventFrameAttribute, bTimeUntilEventKnown, TimeUntilEvent, Weight);
	return true;
}

bool UAnimGenControls::FrameAttributeTrajectoryDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& DirectionFrameAttributes, const float LocationScale, const float LocationWeight, const float DirectionWeight, const FName Tag)
{
	int32 LocationNum = 0, DirectionNum = 0;
	if (!GetTrajectoryControlNum(LocationNum, DirectionNum, Object, Element, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	if (LocationFrameAttributes.Num() != LocationNum || DirectionFrameAttributes.Num() != DirectionNum)
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	TArray<FVector, TInlineAllocator<16>> TrajectoryLocations;
	TArray<FVector, TInlineAllocator<16>> TrajectoryDirections;
	TrajectoryLocations.SetNumUninitialized(LocationNum);
	TrajectoryDirections.SetNumUninitialized(DirectionNum);
	if (!GetTrajectoryControlToArrayViews(TrajectoryLocations, TrajectoryDirections, Object, Element, FTransform::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> FrameAttributes;
	FrameAttributes.SetNum(LocationNum + DirectionNum);

	ParallelFor(LocationNum + DirectionNum, [&](int32 Idx) {
		if (Idx < LocationNum)
		{
			FrameAttributes[Idx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMatchingDistance(LocationFrameAttributes[Idx], TrajectoryLocations[Idx], LocationScale, LocationWeight);
		}
		else
		{
			FrameAttributes[Idx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionMatchingDistance(DirectionFrameAttributes[Idx - LocationNum], TrajectoryDirections[Idx - LocationNum], DirectionWeight);
		}
		});

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSumFromArrayView(FrameAttributes);
	return true;
}

bool UAnimGenControls::FrameAttributePoseDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& LinearVelocityFrameAttributes, const float LocationScale, const float LinearVelocityScale, const float LocationWeight, const float LinearVelocityWeight, const float BlendTime, const FName Tag)
{
	int32 BoneNum = 0;
	if (!GetPoseControlNum(BoneNum, Object, Element, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	if (LocationFrameAttributes.Num() != BoneNum || LinearVelocityFrameAttributes.Num() != BoneNum)
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	TArray<FVector, TInlineAllocator<16>> BoneLocations;
	TArray<FVector, TInlineAllocator<16>> BoneLinearVelocities;
	BoneLocations.SetNumUninitialized(BoneNum);
	BoneLinearVelocities.SetNumUninitialized(BoneNum);
	if (!GetPoseControlToArrayViews(BoneLocations, BoneLinearVelocities, Object, Element, FTransform::Identity, Tag))
	{
		OutFrameAttribute = FAnimDatabaseFrameAttribute();
		return false;
	}

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> FrameAttributes;
	FrameAttributes.SetNum(BoneNum);

	ParallelFor(BoneNum, [&](int32 Idx){

		FrameAttributes[Idx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance(
			LocationFrameAttributes[Idx],
			LinearVelocityFrameAttributes[Idx],
			BoneLocations[Idx],
			BoneLinearVelocities[Idx],
			BlendTime,
			LocationScale,
			LinearVelocityScale,
			LocationWeight,
			LinearVelocityWeight);
	});

	OutFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSumFromArrayView(FrameAttributes);
	return true;
}

FAnimDatabaseFrameRanges UAnimGenControls::ControlSetFrameRanges(const FAnimGenControlSet& ControlSet)
{
	if (!ControlSet.IsValid())
	{
		UE_ANIMGEN_LOG_ONCE(LogAnimGen, Error, "ControlSetFrameRanges: Invalid ControlSet.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Tag;
	Tag.FrameRangeSet = MakeShared<UE::Learning::FFrameRangeSet>(ControlSet.Data->FrameRangeSet);
	return Tag;
}

FAnimGenControlSet UAnimGenControls::MakeEmptyControlSet()
{
	FAnimGenControlSet ControlSet;
	ControlSet.Data = MakeShared<UE::AnimGen::FControlSetData>();
	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeNullControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetTag, const FName Tag)
{
	if (!ControlSetTag.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNullControlSet: Invalid ControlSetTag.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = ControlSetTag.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *ControlSetTag.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeNullControl(Object, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& BoolFrameAttribute, const FName Tag)
{
	if (!BoolFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeBoolControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = BoolFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = BoolFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeBoolControl(Object, BoolFrameAttribute.GetAsBool(FrameIdx), Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeFloatControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const FName Tag)
{
	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeFloatControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = FloatFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = FloatFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeFloatControl(Object, FloatFrameAttribute.GetAsFloat(FrameIdx), Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeLocationControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& LocationFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!LocationFrameAttribute.IsValid() || !RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLocationControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(LocationFrameAttribute, RelativeTransformFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLocationControlSet: LocationFrameAttribute Tag and RelativeTransformFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = LocationFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = LocationFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeLocationControl(
			Object,
			LocationFrameAttribute.GetAsLocationDouble(FrameIdx),
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeRotationControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& RotationFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!RotationFrameAttribute.IsValid() || !RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeRotationControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RotationFrameAttribute, RelativeTransformFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeRotationControlSet: RotationFrameAttribute Tag and RelativeTransformFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = RotationFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = RotationFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeRotationControlFromQuat(
			Object,
			RotationFrameAttribute.GetAsRotationDouble(FrameIdx),
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx).GetRotation(),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeLinearVelocityControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& LinearVelocityFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!LinearVelocityFrameAttribute.IsValid() || !RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLinearVelocityControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(LinearVelocityFrameAttribute, RelativeTransformFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLinearVelocityControlSet: LinearVelocityFrameAttribute Tag and RelativeTransformFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = LinearVelocityFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = LinearVelocityFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeLinearVelocityControl(
			Object, 
			LinearVelocityFrameAttribute.GetAsVelocityDouble(FrameIdx),
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeAngularVelocityControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& AngularVelocityFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!AngularVelocityFrameAttribute.IsValid() || !RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeAngularVelocityControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(AngularVelocityFrameAttribute, RelativeTransformFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeAngularVelocityControlSet: AngularVelocityFrameAttribute Tag and RelativeTransformFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = AngularVelocityFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = AngularVelocityFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeAngularVelocityControl(
			Object,
			FMath::RadiansToDegrees(AngularVelocityFrameAttribute.GetAsVelocityDouble(FrameIdx)),
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeDirectionControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!DirectionFrameAttribute.IsValid() || !RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeDirectionControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(DirectionFrameAttribute, RelativeTransformFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeDirectionControlSet: DirectionFrameAttribute Tag and RelativeTransformFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = DirectionFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = DirectionFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeDirectionControl(
			Object,
			DirectionFrameAttribute.GetAsDirectionDouble(FrameIdx),
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeTimeControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& TimeFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTimeFrameAttribute, const FName Tag)
{
	if (!TimeFrameAttribute.IsValid() || !RelativeTimeFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeTimeControlSet: Invalid FrameAttribute.");
		return FAnimGenControlSet();
	}

	if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(TimeFrameAttribute, RelativeTimeFrameAttribute))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeTimeControlSet: TimeFrameAttribute Tag and RelativeTimeFrameAttribute Tag must be equal.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = TimeFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = TimeFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeTimeControl(
			Object,
			TimeFrameAttribute.GetAsFloat(FrameIdx),
			RelativeTimeFrameAttribute.GetAsFloat(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeExclusiveDiscreteControlSetFromConstant(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetTag, const int32 DiscreteIndex, const int32 Size, const FName Tag)
{
	if (!ControlSetTag.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeExclusiveDiscreteControlSetFromConstant: Invalid ControlSetTag.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = ControlSetTag.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *ControlSetTag.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeExclusiveDiscreteControl(Object, DiscreteIndex, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeTrajectoryControlSet(FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& DirectionFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	return MakeTrajectoryControlSetFromArrayViews(Object, LocationFrameAttributes, DirectionFrameAttributes, RelativeTransformFrameAttribute, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeTrajectoryControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> DirectionFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeTrajectoryControlSetFromArrayViews: Invalid Frame Attribute.");
		return FAnimGenControlSet();
	}

	if (LocationFrameAttributes.Num() != DirectionFrameAttributes.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeTrajectoryControlSetFromArrayViews: Number of Location FrameAttributes and Direction FrameAttributes don't match: {LocationNum} and {DirectionNum}.", LocationFrameAttributes.Num(), DirectionFrameAttributes.Num());
		return FAnimGenControlSet();
	}

	if (LocationFrameAttributes.Num() == 0 || DirectionFrameAttributes.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeTrajectoryControlSetFromArrayViews: Empty FrameAttributes.");
		return FAnimGenControlSet();
	}

	const int32 FrameAttributeNum = LocationFrameAttributes.Num();

	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		if (!LocationFrameAttributes[FrameAttributeIdx].IsValid() || !DirectionFrameAttributes[FrameAttributeIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeTrajectoryControlSetFromArrayViews: Invalid Frame Attribute.");
			return FAnimGenControlSet();
		}

		if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RelativeTransformFrameAttribute, LocationFrameAttributes[FrameAttributeIdx]) ||
			!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RelativeTransformFrameAttribute, DirectionFrameAttributes[FrameAttributeIdx]))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeTrajectoryControlSetFromArrayViews: All FrameAttribute Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet;

	TArray<FVector, TInlineAllocator<32>> Locations;
	TArray<FVector, TInlineAllocator<32>> Directions;
	Locations.SetNumUninitialized(FrameAttributeNum);
	Directions.SetNumUninitialized(FrameAttributeNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
		{
			Locations[FrameAttributeIdx] = LocationFrameAttributes[FrameAttributeIdx].GetAsLocationDouble(FrameIdx);
			Directions[FrameAttributeIdx] = DirectionFrameAttributes[FrameAttributeIdx].GetAsDirectionDouble(FrameIdx);
		}

		ControlSet.Data->Controls[FrameIdx] = MakeTrajectoryControlFromArrayViews(
			Object,
			Locations,
			Directions,
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeLocationTrajectoryControlSet(FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	return MakeLocationTrajectoryControlSetFromArrayView(Object, LocationFrameAttributes, RelativeTransformFrameAttribute, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeLocationTrajectoryControlSetFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLocationTrajectoryControlSetFromArrayView: Invalid Frame Attribute.");
		return FAnimGenControlSet();
	}

	if (LocationFrameAttributes.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeLocationTrajectoryControlSetFromArrayView: Empty FrameAttributes.");
		return FAnimGenControlSet();
	}

	const int32 FrameAttributeNum = LocationFrameAttributes.Num();

	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		if (!LocationFrameAttributes[FrameAttributeIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeLocationTrajectoryControlSetFromArrayView: Invalid Frame Attribute.");
			return FAnimGenControlSet();
		}

		if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RelativeTransformFrameAttribute, LocationFrameAttributes[FrameAttributeIdx]))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeLocationTrajectoryControlSetFromArrayView: All FrameAttribute Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet;

	TArray<FVector, TInlineAllocator<32>> Locations;
	TArray<FVector, TInlineAllocator<32>> Directions;
	Locations.SetNumUninitialized(FrameAttributeNum);
	Directions.SetNumUninitialized(FrameAttributeNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
		{
			Locations[FrameAttributeIdx] = LocationFrameAttributes[FrameAttributeIdx].GetAsLocationDouble(FrameIdx);
		}

		ControlSet.Data->Controls[FrameIdx] = MakeLocationTrajectoryControlFromArrayViews(
			Object,
			Locations,
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakePoseControlSet(FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& LinearVelocityFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	return MakePoseControlSetFromArrayViews(Object, LocationFrameAttributes, LinearVelocityFrameAttributes, RelativeTransformFrameAttribute, Tag);
}

FAnimGenControlSet UAnimGenControls::MakePoseControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> LinearVelocityFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag)
{
	if (!RelativeTransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakePoseControlSetFromArrayViews: Invalid Frame Attribute.");
		return FAnimGenControlSet();
	}

	if (LocationFrameAttributes.Num() != LinearVelocityFrameAttributes.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakePoseControlSetFromArrayViews: Number of Location FrameAttributes and LinearVelocity FrameAttributes don't match: {LocationNum} and {DirectionNum}.", LocationFrameAttributes.Num(), LinearVelocityFrameAttributes.Num());
		return FAnimGenControlSet();
	}

	if (LocationFrameAttributes.Num() == 0 || LinearVelocityFrameAttributes.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakePoseControlSetFromArrayViews: Empty FrameAttributes.");
		return FAnimGenControlSet();
	}

	const int32 FrameAttributeNum = LocationFrameAttributes.Num();

	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		if (!LocationFrameAttributes[FrameAttributeIdx].IsValid() || !LinearVelocityFrameAttributes[FrameAttributeIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakePoseControlSetFromArrayViews: Invalid Frame Attribute.");
			return FAnimGenControlSet();
		}

		if (!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RelativeTransformFrameAttribute, LocationFrameAttributes[FrameAttributeIdx]) ||
			!UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(RelativeTransformFrameAttribute, LinearVelocityFrameAttributes[FrameAttributeIdx]))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakePoseControlSetFromArrayViews: All FrameAttribute Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = RelativeTransformFrameAttribute.FrameAttribute->FrameRangeSet;

	TArray<FVector, TInlineAllocator<32>> Locations;
	TArray<FVector, TInlineAllocator<32>> LinearVelocities;
	Locations.SetNumUninitialized(FrameAttributeNum);
	LinearVelocities.SetNumUninitialized(FrameAttributeNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
		{
			Locations[FrameAttributeIdx] = LocationFrameAttributes[FrameAttributeIdx].GetAsLocationDouble(FrameIdx);
			LinearVelocities[FrameAttributeIdx] = LinearVelocityFrameAttributes[FrameAttributeIdx].GetAsVelocityDouble(FrameIdx);
		}

		ControlSet.Data->Controls[FrameIdx] = MakePoseControlFromArrayViews(
			Object,
			Locations,
			LinearVelocities,
			RelativeTransformFrameAttribute.GetAsTransformDouble(FrameIdx),
			Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeStructControlSet(FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlSet>& Elements, const FName Tag)
{
	const int32 ElementNum = Elements.Num();

	if (ElementNum == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSet: Empty Element Map.");
		return FAnimGenControlSet();
	}

	TArray<FName, TInlineAllocator<32>> Keys;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<32>> Values;
	Elements.GetKeys(Keys);
	Values.SetNum(Keys.Num());

	for (const TPair<FName, FAnimGenControlSet>& Element : Elements)
	{
		if (!Element.Value.IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSet: Invalid Control Set.");
			return FAnimGenControlSet();
		}

		if (!UE::Learning::FrameRangeSet::Equal(Elements.begin()->Value.Data->FrameRangeSet, Element.Value.Data->FrameRangeSet))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSet: All Element Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = Elements.begin()->Value.Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Elements.begin()->Value.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 KeyIdx = 0; KeyIdx < Keys.Num(); KeyIdx++)
		{
			Values[KeyIdx] = Elements[Keys[KeyIdx]].Data->Controls[FrameIdx];
		}

		ControlSet.Data->Controls[FrameIdx] = MakeStructControlFromArrayViews(Object, Keys, Values, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeStructControlSetFromArrays(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, TArray<FAnimGenControlSet>& Elements, const FName Tag)
{
	return MakeStructControlSetFromArrayViews(Object, ElementNames, Elements, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeStructControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, TArrayView<const FAnimGenControlSet> Elements, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSetFromArrayViews: Empty Element Map.");
		return FAnimGenControlSet();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSetFromArrayViews: Element names and Element Counts don't match. Got {ElementNamesNum} and {ElementsNum}", ElementNames.Num(), Elements.Num());
		return FAnimGenControlSet();
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<32>> Values;
	Values.SetNum(Elements.Num());

	for (const FAnimGenControlSet& Element : Elements)
	{
		if (!Element.IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSetFromArrayViews: Invalid Control Set.");
			return FAnimGenControlSet();
		}

		if (!UE::Learning::FrameRangeSet::Equal(Elements.begin()->Data->FrameRangeSet, Element.Data->FrameRangeSet))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeStructControlSetFromArrayViews: All Element Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = Elements.begin()->Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Elements.begin()->Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ElementIdx++)
		{
			Values[ElementIdx] = Elements[ElementIdx].Data->Controls[FrameIdx];
		}

		ControlSet.Data->Controls[FrameIdx] = MakeStructControlFromArrayViews(Object, ElementNames, Values, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeExclusiveUnionControlSet(FAnimGenControlObject& Object, const FName ElementName, const FAnimGenControlSet& Element, const FName Tag)
{
	if (!Element.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeExclusiveUnionControlSet: Invalid Control Set.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = Element.Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Element.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeExclusiveUnionControl(Object, ElementName, Element.Data->Controls[FrameIdx], Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeEmptyInclusiveUnionControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetTag, const FName Tag)
{
	if (!ControlSetTag.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeEmptyInclusiveUnionControlSet: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = ControlSetTag.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *ControlSetTag.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeInclusiveUnionControlFromArrayViews(Object, {}, {}, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeInclusiveUnionControlSet(FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlSet>& Elements, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeInclusiveUnionControlSet: Empty Element Map. Use MakeEmptyInclusiveUnionControlSet instead.");
		return FAnimGenControlSet();
	}

	TArray<FName, TInlineAllocator<32>> Keys;
	TArray<FAnimGenControlObjectElement, TInlineAllocator<32>> Values;
	Elements.GetKeys(Keys);
	Values.SetNum(Keys.Num());

	for (const TPair<FName, FAnimGenControlSet>& Element : Elements)
	{
		if (!Element.Value.IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeInclusiveUnionControlSet: Invalid Control Set.");
			return FAnimGenControlSet();
		}

		if (!UE::Learning::FrameRangeSet::Equal(Elements.begin()->Value.Data->FrameRangeSet, Element.Value.Data->FrameRangeSet))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeInclusiveUnionControlSet: All Element Tags must be equal.");
			return FAnimGenControlSet();
		}
	}

	const int32 FrameNum = Elements.begin()->Value.Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Elements.begin()->Value.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 KeyIdx = 0; KeyIdx < Keys.Num(); KeyIdx++)
		{
			Values[KeyIdx] = Elements[Keys[KeyIdx]].Data->Controls[FrameIdx];
		}

		ControlSet.Data->Controls[FrameIdx] = MakeInclusiveUnionControlFromArrayViews(Object, Keys, Values, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeOptionalNullControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetTag, const FName Tag)
{
	if (!ControlSetTag.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalNullControlSet: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = ControlSetTag.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *ControlSetTag.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeOptionalNullControl(Object, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeOptionalValidControlSet(FAnimGenControlObject& Object, const FAnimGenControlSet& InControlSet, const FName Tag)
{
	if (!InControlSet.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalValidControlSet: Invalid Control Set.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = InControlSet.Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = InControlSet.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeOptionalValidControl(Object, InControlSet.Data->Controls[FrameIdx], Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeOptionalControlSet(FAnimGenControlObject& Object, const FAnimGenControlSet& InControlSet, const FAnimDatabaseFrameAttribute& ValidFrameAttribute, const FName Tag)
{
	if (!InControlSet.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalControlSet: Invalid Control Set.");
		return FAnimGenControlSet();
	}

	if (!ValidFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalControlSet: Invalid Frame Attribute.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = InControlSet.Data->FrameRangeSet.GetTotalFrameNum();

	if (!UE::Learning::FrameRangeSet::Equal(InControlSet.Data->FrameRangeSet, ValidFrameAttribute.FrameAttribute->FrameRangeSet))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalControlSet: Control Set and ValidFrameAttribute Tags must be equal.");
		return FAnimGenControlSet();
	}

	if (ValidFrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeOptionalControlSet: ValidFrameAttribute must be of Bool type.");
		return FAnimGenControlSet();
	}

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = InControlSet.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		if (ValidFrameAttribute.GetAsBool(FrameIdx))
		{
			ControlSet.Data->Controls[FrameIdx] = MakeOptionalValidControl(Object, InControlSet.Data->Controls[FrameIdx], Tag);
		}
		else
		{
			ControlSet.Data->Controls[FrameIdx] = MakeOptionalNullControl(Object, Tag);
		}
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeEnumControlSet(FAnimGenControlObject& Object, const UEnum* Enum, const uint8 EnumValue, const FAnimDatabaseFrameRanges& ControlSetTag, const FName Tag)
{
	if (!ControlSetTag.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeEnumControlSet: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = ControlSetTag.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *ControlSetTag.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeEnumControl(Object, Enum, EnumValue, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeEventControlSet(FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& EventFrameAttribute, const FName Tag)
{
	if (!EventFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeEventControlSet: Invalid Frame Attribute.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = EventFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = EventFrameAttribute.FrameAttribute->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		bool bTimeUntilEventKnown = false;
		float TimeUntilEvent = UE_MAX_FLT;
		EventFrameAttribute.GetAsEvent(bTimeUntilEventKnown, TimeUntilEvent, FrameIdx);

		ControlSet.Data->Controls[FrameIdx] = MakeEventControl(Object, bTimeUntilEventKnown, TimeUntilEvent, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeSetControlSetFromArray(FAnimGenControlObject& Object, const TArray<FAnimGenControlSet>& Elements, const TArray<FAnimDatabaseFrameAttribute>& ElementMasks, const FName Tag)
{
	return MakeSetControlSetFromArrayView(Object, Elements, ElementMasks, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeSetControlSetFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlSet> Elements, const TArrayView<const FAnimDatabaseFrameAttribute> ElementMasks, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: Empty Set.");
		return FAnimGenControlSet();
	}

	TArray<FAnimGenControlObjectElement, TInlineAllocator<32>> SetControls;
	SetControls.Reserve(Elements.Num());

	const int32 ElementNum = Elements.Num();

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!Elements[ElementIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: Invalid Control Set.");
			return FAnimGenControlSet();
		}

		if (!UE::Learning::FrameRangeSet::Equal(Elements.begin()->Data->FrameRangeSet, Elements[ElementIdx].Data->FrameRangeSet))
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: All Element Tags must be equal.");
			return FAnimGenControlSet();
		}

		if (ElementIdx < ElementMasks.Num())
		{
			if (!ElementMasks[ElementIdx].IsValid())
			{
				UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: Invalid Frame Attribute.");
				return FAnimGenControlSet();
			}

			if (ElementMasks[ElementIdx].Type != EAnimDatabaseAttributeType::Bool)
			{
				UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: Element Masks FrameAttributes must be of Bool type.");
				return FAnimGenControlSet();
			}

			if (!UE::Learning::FrameRangeSet::Equal(Elements.begin()->Data->FrameRangeSet, ElementMasks[ElementIdx].FrameAttribute->FrameRangeSet))
			{
				UE_LOGFMT(LogAnimGen, Error, "MakeSetControlSetFromArrayView: All Element Tags must be equal.");
				return FAnimGenControlSet();
			}
		}
	}

	const int32 FrameNum = Elements.begin()->Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Elements.begin()->Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		SetControls.Reset();
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ElementIdx++)
		{
			if (ElementIdx < ElementMasks.Num())
			{
				if (ElementMasks[ElementIdx].GetAsBool(FrameIdx))
				{
					SetControls.Add(Elements[ElementIdx].Data->Controls[FrameIdx]);
				}
			}
			else
			{
				SetControls.Add(Elements[ElementIdx].Data->Controls[FrameIdx]);
			}
		}

		ControlSet.Data->Controls[FrameIdx] = MakeSetControlFromArrayView(Object, SetControls, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeEncodingControlSet(FAnimGenControlObject& Object, const FAnimGenControlSet& Element, const FName Tag)
{
	if (!Element.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeEncodingControlSet: Invalid Control Set.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = Element.Data->FrameRangeSet.GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = Element.Data->FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeEncodingControl(Object,Element.Data->Controls[FrameIdx], Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeNamedInclusiveDiscreteControlSetFromFrameRanges(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameRanges>& ElementMap, const FName Tag)
{
	TArray<FName, TInlineAllocator<16>> ElementNames;
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<16>> ElementFrameRanges;
	for (const TPair<FName, FAnimDatabaseFrameRanges>& Element : ElementMap)
	{
		ElementNames.Add(Element.Key);
		ElementFrameRanges.Add(Element.Value);
	}
	return MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews(Object, FrameRanges, ElementNames, ElementFrameRanges, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameRanges> ElementFrameRanges, const FName Tag)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	if (ElementNames.Num() != ElementFrameRanges.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews: ElementNames and ElementFrameRanges must match in size.");
		return FAnimGenControlSet();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSet();
	}

	const int32 ElementNum = ElementNames.Num();

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!ElementFrameRanges[ElementIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
			return FAnimGenControlSet();
		}
	}

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *FrameRanges.FrameRangeSet;

	TArray<FAnimDatabaseFrameAttribute> ElementAttributes;
	ElementAttributes.SetNum(ElementNum);
	UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromActiveRangesArrayViews(ElementAttributes, ElementFrameRanges, FrameRanges);

	TArray<FName, TInlineAllocator<32>> Names;
	Names.Reserve(ElementNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		Names.Reset();
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			if (ElementAttributes[ElementIdx].GetAsBool(FrameIdx))
			{
				Names.Add(ElementNames[ElementIdx]);
			}
		}

		ControlSet.Data->Controls[FrameIdx] = MakeNamedInclusiveDiscreteControlFromArrayView(Object, Names, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeNamedInclusiveDiscreteControlSetFromAnimNotifyStates(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, TSubclassOf<UAnimNotifyState>>& AnimNotifyStateMap, const FName Tag)
{
	TArray<FName, TInlineAllocator<16>> ElementNames;
	TArray<TSubclassOf<UAnimNotifyState>, TInlineAllocator<16>> ElementAnimNotifyStates;
	for (const TPair<FName, TSubclassOf<UAnimNotifyState>>& Element : AnimNotifyStateMap)
	{
		ElementNames.Add(Element.Key);
		ElementAnimNotifyStates.Add(Element.Value);
	}
	return MakeNamedInclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(Object, Database, FrameRanges, ElementNames, ElementAnimNotifyStates, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedInclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> ElementAnimNotifyStates, const FName Tag)
{
	const int32 ElementNum = ElementAnimNotifyStates.Num();
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<16>> ElementFrameRanges;
	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		ElementFrameRanges.Add(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, ElementAnimNotifyStates[ElementIdx]));
	}
	return MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews(Object, FrameRanges, ElementNames, ElementFrameRanges, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedExclusiveDiscreteControlSetFromFrameRanges(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameRanges>& ElementMap, const FName DefaultElementName, const FName Tag)
{
	TArray<FName, TInlineAllocator<16>> ElementNames;
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<16>> ElementFrameRanges;
	for (const TPair<FName, FAnimDatabaseFrameRanges>& Element : ElementMap)
	{
		ElementNames.Add(Element.Key);
		ElementFrameRanges.Add(Element.Value);
	}
	return MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews(Object, FrameRanges, ElementNames, ElementFrameRanges, DefaultElementName, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameRanges> ElementFrameRanges, const FName DefaultElementName, const FName Tag)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	if (ElementNames.Num() != ElementFrameRanges.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews: ElementNames and ElementFrameRanges must match in size.");
		return FAnimGenControlSet();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSet();
	}

	const int32 ElementNum = ElementNames.Num();

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!ElementFrameRanges[ElementIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
			return FAnimGenControlSet();
		}
	}

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *FrameRanges.FrameRangeSet;

	TArray<FAnimDatabaseFrameAttribute> ElementAttributes;
	ElementAttributes.SetNum(ElementNum);
	UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromActiveRangesArrayViews(ElementAttributes, ElementFrameRanges, FrameRanges);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		bool bElementFound = false;
		FName ElementName = NAME_None;
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			if (ElementAttributes[ElementIdx].GetAsBool(FrameIdx))
			{
				bElementFound = true;
				ElementName = ElementNames[ElementIdx];
				break;
			}
		}

		ControlSet.Data->Controls[FrameIdx] = MakeNamedExclusiveDiscreteControl(Object, bElementFound ? ElementName : DefaultElementName, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeNamedExclusiveDiscreteControlSetFromAnimNotifyStates(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, TSubclassOf<UAnimNotifyState>>& AnimNotifyStateMap, const FName DefaultElementName, const FName Tag)
{
	TArray<FName, TInlineAllocator<16>> ElementNames;
	TArray<TSubclassOf<UAnimNotifyState>, TInlineAllocator<16>> ElementAnimNotifyStates;
	for (const TPair<FName, TSubclassOf<UAnimNotifyState>>& Element : AnimNotifyStateMap)
	{
		ElementNames.Add(Element.Key);
		ElementAnimNotifyStates.Add(Element.Value);
	}
	return MakeNamedExclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(Object, Database, FrameRanges, ElementNames, ElementAnimNotifyStates, DefaultElementName, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedExclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> ElementAnimNotifyStates, const FName DefaultElementName, const FName Tag)
{
	const int32 ElementNum = ElementAnimNotifyStates.Num();
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<16>> ElementFrameRanges;
	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		ElementFrameRanges.Add(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, ElementAnimNotifyStates[ElementIdx]));
	}
	return MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews(Object, FrameRanges, ElementNames, ElementFrameRanges, DefaultElementName, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedSparseControlSetFromFrameRanges(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameAttribute>& ElementMap, const FName Tag)
{
	TArray<FName, TInlineAllocator<16>> ElementNames;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<16>> ElementAttributes;
	for (const TPair<FName, FAnimDatabaseFrameAttribute>& Element : ElementMap)
	{
		ElementNames.Add(Element.Key);
		ElementAttributes.Add(Element.Value);
	}

	return MakeNamedSparseControlSetFromFrameRangesArrayViews(Object, FrameRanges, ElementNames, ElementAttributes, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeNamedSparseControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameAttribute> ElementFrameValues, const FName Tag)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedSparseControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	if (ElementNames.Num() != ElementFrameValues.Num())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedSparseControlSetFromFrameRangesArrayViews: ElementNames and ElementFrameValues must match in size.");
		return FAnimGenControlSet();
	}

	if (UE::AnimGen::Control::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeNamedSparseControlSetFromFrameRangesArrayViews: Element Names contain duplicates.");
		return FAnimGenControlSet();
	}

	const int32 ElementNum = ElementNames.Num();

	for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		if (!ElementFrameValues[ElementIdx].IsValid())
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeNamedSparseControlSetFromFrameRangesArrayViews: Invalid Frame Ranges.");
			return FAnimGenControlSet();
		}

		if (ElementFrameValues[ElementIdx].Type != EAnimDatabaseAttributeType::Float)
		{
			UE_LOGFMT(LogAnimGen, Error, "MakeNamedSparseControlSetFromFrameRangesArrayViews: All attributes must be of type Float, got {Type}.", 
				UAnimDatabaseFrameAttributeLibrary::AttributeTypeNameInternal(ElementFrameValues[ElementIdx].Type));
			return FAnimGenControlSet();
		}
	}

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *FrameRanges.FrameRangeSet;

	TArray<FName, TInlineAllocator<32>> Names;
	TArray<float, TInlineAllocator<32>> Values;
	Names.Reserve(ElementNum);
	Values.Reserve(ElementNum);

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

		for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
		{
			const int32 RangeStart = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			const int32 RangeFrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 RangeOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);

			for (int32 RangeFrameIdx = 0; RangeFrameIdx < RangeFrameNum; RangeFrameIdx++)
			{
				Names.Reset();
				Values.Reset();
				for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
				{
					int32 ElementEntryIdx = INDEX_NONE, ElementRangeIdx = INDEX_NONE, ElementRangeFrame = INDEX_NONE;
					if (ElementFrameValues[ElementIdx].FrameAttribute->FrameRangeSet.Find(ElementEntryIdx, ElementRangeIdx, ElementRangeFrame, SequenceIdx, RangeStart + RangeFrameIdx))
					{
						const int32 ElementFrame = ElementFrameValues[ElementIdx].FrameAttribute->FrameRangeSet.GetEntryRangeOffset(ElementEntryIdx, ElementRangeIdx) + ElementRangeFrame;

						Names.Add(ElementNames[ElementIdx]);
						Values.Add(ElementFrameValues[ElementIdx].GetAsFloat(ElementFrame));
					}
				}

				ControlSet.Data->Controls[RangeOffset + RangeFrameIdx] = MakeNamedSparseControlFromArrayView(Object, Names, Values, Tag);
			}
		}
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeRootLinearVelocityControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeLinearVelocityControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(Database, FrameRanges),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeRootAngularVelocityControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeAngularVelocityControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootAngularVelocityFrameAttribute(Database, FrameRanges),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeFutureRootLinearVelocityControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float FutureTime, const FName Tag)
{
	return MakeLinearVelocityControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(Database, FrameRanges, FutureTime),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeFutureRootDirectionControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float FutureTime, const FVector ForwardVector, const FName Tag)
{
	return MakeDirectionControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionFrameAttribute(Database, FrameRanges, FutureTime, ForwardVector),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeRootLocationAtRangeEndControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeLocationControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAtRangeEndFrameAttribute(Database, FrameRanges),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeRootDirectionAtRangeEndControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FVector ForwardVector, const FName Tag)
{
	return MakeDirectionControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionAtRangeEndFrameAttribute(Database, FrameRanges, ForwardVector),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeTrajectoryControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 SampleNum, const float FutureTime, const float PastTime, const FVector ForwardVector, const FName Tag)
{
	TArray<FAnimDatabaseFrameAttribute> TrajectoryLocations;
	TArray<FAnimDatabaseFrameAttribute> TrajectoryDirections;
	UAnimDatabaseFrameAttributeLibrary::MakeTrajectoryFrameAttribute(TrajectoryLocations, TrajectoryDirections, Database, FrameRanges, SampleNum, FutureTime, PastTime, ForwardVector);

	return MakeTrajectoryControlSet(Object, 
		TrajectoryLocations, 
		TrajectoryDirections, 
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeLookAtTrajectoryControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 LookAtBoneIndex, const int32 SampleNum, const float FutureTime, const float PastTime, const FVector LocalDirection, const float DirectionSmoothingAmount, const FName Tag)
{
	TArray<FAnimDatabaseFrameAttribute> TrajectoryLocations;
	TArray<FAnimDatabaseFrameAttribute> TrajectoryDirections;
	UAnimDatabaseFrameAttributeLibrary::MakeLookAtTrajectoryFrameAttribute(TrajectoryLocations, TrajectoryDirections, Database, FrameRanges, LookAtBoneIndex, SampleNum, FutureTime, PastTime, LocalDirection, DirectionSmoothingAmount);

	return MakeTrajectoryControlSet(Object,
		TrajectoryLocations,
		TrajectoryDirections,
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeLocationTrajectoryControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 SampleNum, const float FutureTime, const float PastTime, const FName Tag)
{
	TArray<FAnimDatabaseFrameAttribute> TrajectoryLocations;
	UAnimDatabaseFrameAttributeLibrary::MakeLocationTrajectoryFrameAttribute(TrajectoryLocations, Database, FrameRanges, SampleNum, FutureTime, PastTime);

	return MakeLocationTrajectoryControlSet(Object,
		TrajectoryLocations,
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSetFromConstant(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const bool bValue, const FName Tag)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimGen, Error, "MakeBoolControlSetFromConstant: Invalid Frame Ranges.");
		return FAnimGenControlSet();
	}

	const int32 FrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	FAnimGenControlSet ControlSet = MakeEmptyControlSet();
	ControlSet.Data->Controls.SetNumUninitialized(FrameNum);
	ControlSet.Data->FrameRangeSet = *FrameRanges.FrameRangeSet;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		ControlSet.Data->Controls[FrameIdx] = MakeBoolControl(Object, bValue, Tag);
	}

	return ControlSet;
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSetFromMirrored(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeBoolControlSet(Object, UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromMirrored(Database, FrameRanges), Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSetFromNotMirrored(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeBoolControlSet(Object, UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromNotMirrored(Database, FrameRanges), Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSetFromActiveRanges(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& Active, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeBoolControlSet(Object, UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(Active, FrameRanges), Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoolControlSetFromAnimNotifyState(FAnimGenControlObject& Object, const UAnimDatabase* Database, const TSubclassOf<UAnimNotifyState> AnimNotifyState, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag)
{
	return MakeBoolControlSetFromActiveRanges(Object, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, AnimNotifyState), FrameRanges, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalLocationControlSet(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FName Tag)
{
	return MakeLocationControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttribute(Database, FrameRanges, BoneIndex),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalLocationControlSetFromName(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FName Tag)
{
	return MakeBoneGlobalLocationControlSet(Object, Database, FrameRanges, Database->FindBoneIndex(BoneName), Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalLocationAtNearestFramesControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FAnimDatabaseFrames& Frames, const FName Tag)
{
	return MakeLocationControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAtNearestFramesFrameAttribute(Database, FrameRanges, BoneIndex, Frames),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalLocationAtNearestFramesControlSetFromDatabaseBoneName(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FAnimDatabaseFrames& Frames, const FName Tag)
{
	return MakeBoneGlobalLocationAtNearestFramesControlSetFromDatabase(Object, Database, FrameRanges, Database->FindBoneIndex(BoneName), Frames, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalDirectionAtNearestFramesControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FAnimDatabaseFrames& Frames, const FVector LocalDirection, const FName Tag)
{
	return MakeDirectionControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionAtNearestFramesFrameAttribute(Database, FrameRanges, BoneIndex, Frames, LocalDirection),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalDirectionAtNearestFramesControlSetFromDatabaseBoneName(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FAnimDatabaseFrames& Frames, const FVector LocalDirection, const FName Tag)
{
	return MakeBoneGlobalDirectionAtNearestFramesControlSetFromDatabase(Object, Database, FrameRanges, Database->FindBoneIndex(BoneName), Frames, LocalDirection, Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalDirectionControlSet(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FVector ForwardVector, const FName Tag)
{
	return MakeDirectionControlSet(Object,
		UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttribute(Database, FrameRanges, BoneIndex, ForwardVector, 0.0f),
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges),
		Tag);
}

FAnimGenControlSet UAnimGenControls::MakeBoneGlobalDirectionControlSetFromName(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FVector ForwardVector, const FName Tag)
{
	return MakeBoneGlobalDirectionControlSet(Object, Database, FrameRanges, Database->FindBoneIndex(BoneName), ForwardVector, Tag);
}

FAnimGenControlSet UAnimGenControls::MakePoseControlSetFromDatabase(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const FName Tag)
{
	return MakePoseControlSetFromDatabaseArrayView(Object, Database, FrameRanges, BoneIndices, Tag);
}

FAnimGenControlSet UAnimGenControls::MakePoseControlSetFromDatabaseArrayView(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const FName Tag)
{
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<8>> BoneLocationAttributes;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<8>> BoneVelocityAttributes;
	BoneLocationAttributes.SetNum(BoneIndices.Num());
	BoneVelocityAttributes.SetNum(BoneIndices.Num());
	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributesFromArrayViews(
		BoneLocationAttributes, {}, {},
		BoneVelocityAttributes, {}, {},
		Database,
		FrameRanges,
		BoneIndices, {}, {},
		BoneIndices, {}, {});

	return MakePoseControlSetFromArrayViews(Object,
		BoneLocationAttributes,
		BoneVelocityAttributes,
		UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(Database, FrameRanges), Tag);
}

FAnimGenControlSet UAnimGenControls::MakeTimeUntilNearestAnimNotifyControlSet(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, TSubclassOf<UAnimNotify> AnimNotifyClass, const FName Tag)
{
	return MakeTimeControlSet(
		Object,
		UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromNearestFrameSequenceTimes(
			FrameRanges, UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(Database, FrameRanges, AnimNotifyClass), Database->GetFrameRate()),
		UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromSequenceTimes(FrameRanges, Database->GetFrameRate()),
		Tag);
}

bool UAnimGenControls::DrawDebugLocationControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FLinearColor DrawColor, const float DrawRadius, const float Thickness, const int32 SegmentNum, const FName Tag)
{
	FVector Location;
	if (!GetLocationControl(Location, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugSimpleSphere(Drawer, Location, FRotator::ZeroRotator, LineStyle, true, DrawRadius);
	return true;
}

bool UAnimGenControls::DrawDebugRotationControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FVector DrawLocation, const float DrawRadius, const float Thickness, const FName Tag)
{
	FQuat Rotation;
	if (!GetRotationControlAsQuat(Rotation, Object, Element, RelativeTransform.GetRotation(), Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = FLinearColor::White;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugRotation(Drawer, DrawLocation, Rotation.Rotator(), LineStyle, true, DrawRadius);
	return true;
}

bool UAnimGenControls::DrawDebugLinearVelocityControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FVector DrawLocation, const FLinearColor DrawColor, const float DrawVelocityScale, const float Thickness, const FName Tag)
{
	FVector LinearVelocity;
	if (!GetLinearVelocityControl(LinearVelocity, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugLine(Drawer, DrawLocation, DrawLocation + DrawVelocityScale * LinearVelocity, LineStyle);
	return true;
}

bool UAnimGenControls::DrawDebugAngularVelocityControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FVector DrawLocation, const FLinearColor DrawColor, const float DrawVelocityScale, const float Thickness, const FName Tag)
{
	FVector AngularVelocity;
	if (!GetAngularVelocityControl(AngularVelocity, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugLine(Drawer, DrawLocation, DrawLocation + DrawVelocityScale * AngularVelocity, LineStyle);
	return true;
}

bool UAnimGenControls::DrawDebugDirectionControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FVector DrawLocation, const FLinearColor DrawColor, const float DrawArrowLength, const float ArrowHeadScale, const float Thickness, const FName Tag)
{
	FVector Direction;
	if (!GetDirectionControl(Direction, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugDirection(Drawer, DrawLocation, Direction, LineStyle, true, DrawArrowLength, ArrowHeadScale);
	return true;
}

bool UAnimGenControls::DrawDebugPoseControlFromPoseStateArrayViews(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabasePoseState& PoseState, const TArrayView<const int32> BoneIndices, const FLinearColor DrawColor, const float DrawVelocityLineScale, const float Thickness, const FName Tag)
{
	const int32 BoneNum = BoneIndices.Num();

	const FTransform RelativeTransform = UAnimDatabasePoseStateLibrary::PoseStateRootTransform(PoseState);

	TArray<FVector, TInlineAllocator<16>> Locations;
	TArray<FVector, TInlineAllocator<16>> LinearVelocities;
	Locations.SetNumUninitialized(BoneNum);
	LinearVelocities.SetNumUninitialized(BoneNum);
	if (!GetPoseControlToArrayViews(Locations, LinearVelocities, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		UDrawDebugLibrary::DrawDebugSimpleSphere(Drawer, Locations[BoneIdx], FRotator::ZeroRotator, LineStyle, true, 10.0f);
		UDrawDebugLibrary::DrawDebugLine(Drawer, Locations[BoneIdx], Locations[BoneIdx] + DrawVelocityLineScale * LinearVelocities[BoneIdx], LineStyle);
	}
	return true;
}

bool UAnimGenControls::DrawDebugPoseControlFromPoseState(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabasePoseState& PoseState, const TArray<int32>& BoneIndices, const FLinearColor DrawColor, const float DrawVelocityLineScale, const float Thickness, const FName Tag)
{
	return DrawDebugPoseControlFromPoseStateArrayViews(Drawer, Object, Element, PoseState, BoneIndices, DrawColor, DrawVelocityLineScale, Thickness, Tag);
}

bool UAnimGenControls::DrawDebugTrajectoryControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FLinearColor DrawColor, const float DrawArrowLength, const float PointRadius, const float ArrowHeadScale, const float Thickness, const int32 SegmentNum, const FName Tag)
{
	TArray<FVector, TInlineAllocator<16>> Locations;
	TArray<FVector, TInlineAllocator<16>> Directions;

	int32 LocationNum = 0, DirectionNum = 0;
	if (!GetTrajectoryControlNum(LocationNum, DirectionNum, Object, Element, Tag))
	{
		return false;
	}

	if (!ensure(LocationNum == DirectionNum))
	{
		return false;
	}

	Locations.SetNumUninitialized(LocationNum);
	Directions.SetNumUninitialized(DirectionNum);
	if (!GetTrajectoryControlToArrayViews(Locations, Directions, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugTrajectoryFromArrayViews(
		Drawer, Locations, Directions, FTransform::Identity, LineStyle, true, DrawArrowLength, PointRadius, ArrowHeadScale, SegmentNum);

	return true;
}

bool UAnimGenControls::DrawDebugLocationTrajectoryControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform, const FLinearColor DrawColor, const float PointRadius, const float Thickness, const int32 SegmentNum, const FName Tag)
{
	TArray<FVector, TInlineAllocator<16>> Locations;

	int32 LocationNum = 0;
	if (!GetLocationTrajectoryControlNum(LocationNum, Object, Element, Tag))
	{
		return false;
	}

	Locations.SetNumUninitialized(LocationNum);
	if (!GetLocationTrajectoryControlToArrayViews(Locations, Object, Element, RelativeTransform, Tag))
	{
		return false;
	}

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;
	LineStyle.Thickness = Thickness;

	UDrawDebugLibrary::DrawDebugLocationsArrayView(Drawer, Locations, LineStyle, true, PointRadius, SegmentNum);

	return true;
}

bool UAnimGenControls::DrawDebugNamedExclusiveDiscreteControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale, const FLinearColor DrawColor, const float Thickness, const FName Tag)
{
	FName Name = NAME_None;
	if (!GetNamedExclusiveDiscreteControl(Name, Object, Element, Tag))
	{
		return false;
	}

	FDrawDebugStringSettings Settings;
	Settings.Height = Scale;
	Settings.bCenterHorizontally = true;
	Settings.bCenterVertically = true;

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;

	FString String = Name.ToString();

	if (Label != NAME_None)
	{
		String = Label.ToString() + String;
	}

	UDrawDebugLibrary::DrawDebugString(Drawer, String, Location, Rotation, LineStyle, true, Settings);
	return true;
}

bool UAnimGenControls::DrawDebugNamedInclusiveDiscreteControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale, const FLinearColor DrawColor, const float Thickness, const FName Tag)
{
	TArray<FName, TInlineAllocator<32>> Names;
	int32 NameNum = 0;
	if (!GetNamedInclusiveDiscreteControlNum(NameNum, Object, Element, Tag))
	{
		return false;
	}

	Names.Init(NAME_None, NameNum);
	if (!GetNamedInclusiveDiscreteControlToArrayView(Names, Object, Element, Tag))
	{
		return false;
	}
	FString String;
	for (int32 NameIdx = 0; NameIdx < NameNum; NameIdx++)
	{
		String = String.Append(Names[NameIdx].ToString());
		if (NameIdx != NameNum - 1)
		{
			String = String.Append(TEXT(" | "));
		}
	}

	FDrawDebugStringSettings Settings;
	Settings.Height = Scale;
	Settings.bCenterHorizontally = true;
	Settings.bCenterVertically = true;

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;

	if (Label != NAME_None)
	{
		String = Label.ToString() + String;
	}

	UDrawDebugLibrary::DrawDebugString(Drawer, String, Location, Rotation, LineStyle, true, Settings);
	return true;
}

bool UAnimGenControls::DrawDebugNamedSparseControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale, const FLinearColor DrawColor, const float Thickness, const FName Tag)
{
	TArray<FName, TInlineAllocator<32>> Names;
	TArray<float, TInlineAllocator<32>> Values;
	int32 NameNum = 0;
	if (!GetNamedSparseControlNum(NameNum, Object, Element, Tag))
	{
		return false;
	}

	Names.Init(NAME_None, NameNum);
	Values.Init(0.0f, NameNum);
	if (!GetNamedSparseControlToArrayView(Names, Values, Object, Element, Tag))
	{
		return false;
	}
	FString String;
	for (int32 NameIdx = 0; NameIdx < NameNum; NameIdx++)
	{
		String = String.Append(FString::Printf(TEXT("%s: % 7.2f"), *Names[NameIdx].ToString(), Values[NameIdx]));
		if (NameIdx != NameNum - 1)
		{
			String = String.Append(TEXT(" | "));
		}
	}

	FDrawDebugStringSettings Settings;
	Settings.Height = Scale;
	Settings.bCenterHorizontally = true;
	Settings.bCenterVertically = true;

	FDrawDebugLineStyle LineStyle;
	LineStyle.Color = DrawColor;

	if (Label != NAME_None)
	{
		String = Label.ToString() + String;
	}

	UDrawDebugLibrary::DrawDebugString(Drawer, String, Location, Rotation, LineStyle, true, Settings);
	return true;
}

#undef UE_ANIMGEN_LOG_ONCE