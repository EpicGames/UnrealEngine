// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenController.h"

#include "AnimGenLog.h"
#include "AnimGenControl.h"
#include "AnimGenAutoEncoder.h"
#include "AnimGenBehavior.h"

#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseMath.h"

#include "UObject/Package.h"
#include "Misc/ScopedSlowTask.h"

#include "Animation/Skeleton.h"

#include "LearningNeuralNetwork.h"

#include "DrawDebugLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGenController)

#define LOCTEXT_NAMESPACE "AnimGenController"

#if WITH_EDITOR

void UAnimGenControllerTrainingSettings::ForceRefresh()
{
	bForceRefresh = true;
}

void UAnimGenControllerTrainingSettings::LoadDatabaseAsync()
{
	Database.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda([this](const FSoftObjectPath&, UObject*) { ForceRefresh(); }));
}

void UAnimGenControllerTrainingSettings::LoadAutoEncoderAsync()
{
	AutoEncoder.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda([this](const FSoftObjectPath&, UObject*) { ForceRefresh(); }));
}

void UAnimGenControllerTrainingSettings::PostEditChangeProperty(struct FPropertyChangedEvent& Event)
{
	if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, Database))
	{
		ForceRefresh();
		LoadDatabaseAsync();
	}
	else if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, AutoEncoder))
	{
		ForceRefresh();
		LoadAutoEncoderAsync();
	}
}

void UAnimGenControllerTrainingSettings::UpdateQueryEntries()
{
	if (Database && AutoEncoder)
	{
		const int32 ControlSetNum = ControlSets.Num();

		int32 TotalRangeNum = 0;
		for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
		{
			check(ControlSets[ControlSetIdx].IsValid());
			TotalRangeNum += ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalRangeNum();
		}

		QueryEntries.SetNum(TotalRangeNum);
		int32 QueryEntryIdx = 0;

		for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
		{
			const UE::Learning::FFrameRangeSet& FrameRangeSet = ControlSets[ControlSetIdx].Data->FrameRangeSet;

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 EntrySequenceIdx = FrameRangeSet.GetEntrySequence(EntryIdx);
				const UAnimSequence* AnimSequence = Database->GetAnimSequence(EntrySequenceIdx);
				const bool bIsMirrored = Database->GetIsMirrored(EntrySequenceIdx);

				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				for (int32 EntryRangeIdx = 0; EntryRangeIdx < RangeNum; EntryRangeIdx++)
				{
					TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>& QueryEntry = QueryEntries[QueryEntryIdx];
					if (!QueryEntry.IsValid()) { QueryEntry = MakeShared<UE::AnimDatabase::Editor::FQueryEntry>(); }

					const int32 NewStartFrame = FrameRangeSet.GetEntryRangeStart(EntryIdx, EntryRangeIdx);
					const int32 NewStopFrame = FrameRangeSet.GetEntryRangeStop(EntryIdx, EntryRangeIdx) - 1;

					// If the entry has changed then reset selection

					if (QueryEntry->Sequence != AnimSequence ||
						QueryEntry->StartFrame != NewStartFrame ||
						QueryEntry->StopFrame != NewStopFrame ||
						QueryEntry->bIsMirrored != bIsMirrored)
					{
						QueryEntry->Sequence = AnimSequence;
						QueryEntry->StartFrame = NewStartFrame;
						QueryEntry->StopFrame = NewStopFrame;
						QueryEntry->bIsMirrored = bIsMirrored;
						QueryEntry->bIsSelected = false;
					}

					// Here we use a hash of the sequence, start, and stop frames to produce the color

					const uint32 HashData[6] =
					{
						(uint32)RangeIndentifierColorSeed,
						(uint32)ControlSetIdx,
						GetTypeHash(AnimSequence),
						(uint32)QueryEntry->StartFrame,
						(uint32)QueryEntry->StopFrame,
						(uint32)QueryEntry->bIsMirrored
					};

					const uint32 ColorHash = CityHash32((const char*)HashData, sizeof(HashData));
					const uint8 Hue = ColorHash & 0x000000FF;
					const uint8 Saturation = ((ColorHash & 0x0000FF00) >> 8) % 25 + 230;
					const uint8 Brightness = ((ColorHash & 0x00FF0000) >> 16) % 50 + 205;

					QueryEntry->Color = FLinearColor::MakeFromHSV8(Hue, Saturation, Brightness);

					QueryEntryIdx++;
				}
			}
		}

		check(QueryEntryIdx == QueryEntries.Num());
	}
	else
	{
		QueryEntries.Reset();
	}
}

bool UAnimGenControllerTrainingSettings::Update()
{
	if (const UAnimDatabase* DatabaseObject = Database.Get())
	{
		bool bControlSetsUpdateRequired = bForceRefresh;
		bForceRefresh = false;

		// Check if Database has been updated

		const uint32 CurrentContentHash = DatabaseObject->GetContentHash();
		if (CurrentContentHash != DatabaseContentHash)
		{
			DatabaseContentHash = CurrentContentHash;
			bControlSetsUpdateRequired = true;
		}

		// Check if FrameRanges Class is out-of-date and the class has been regenerated

		if (bControlSetsUpdateRequired)
		{
			// Rebuild Frame Ranges

			if (!FrameRanges)
			{
				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(DatabaseObject);
			}
			else
			{
				FScopedSlowTask SlowTaskRanges(0.0f, LOCTEXT("RebuildingTrainingRanges", "Rebuilding Training Frame Ranges"));
				SlowTaskRanges.MakeDialog();

				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(DatabaseObject, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(DatabaseObject), FrameRanges);
				if (!QueryRanges.IsValid()) { QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges(); }
			}

			// Rebuild Control Sets

			if (Behavior)
			{
				FScopedSlowTask SlowTaskControls(0.0f, LOCTEXT("RebuildingControlSets", "Rebuilding Control Sets"));
				SlowTaskControls.MakeDialog();

				ControlSchema = UAnimGenControls::MakeControlSchema();
				ControlObject = UAnimGenControls::MakeControlObject();
				ControlSchemaElement = FAnimGenControlSchemaElement();
				ControlSets.Empty();
				ControlSchemaElement = Behavior->SpecifyControl(ControlSchema);
				if (ControlSchema.ObservationSchema->Schema.IsValid(ControlSchemaElement.SchemaElement))
				{
					Behavior->MakeControlSets(ControlSets, ControlObject, DatabaseObject, QueryRanges);
					const int32 ControlSetNum = ControlSets.Num();
					for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
					{
						if (!ControlSets[ControlSetIdx].IsValid())
						{
							ControlSets.Empty();
							break;
						}
					}
				}
				else
				{
					ControlSets.Empty();
				}

				// Update Debug Drawer

				if (DebugDrawer)
				{
					FScopedSlowTask SlowTaskDebugDraw(0.0f, LOCTEXT("ReinitializingDebugDrawData", "Reinitializing Debug Draw"));
					SlowTaskDebugDraw.MakeDialog();

					DebugDrawer->InitializeDrawDebug(DatabaseObject, QueryRanges);
				}
			}
			else
			{
				ControlSets.Empty();
			}

			UpdateQueryEntries();
		}

		// Update Selection

		SelectedRanges.Reset();

		const int32 QueryRangeNum = QueryEntries.Num();

		for (int32 QueryRangeIdx = 0; QueryRangeIdx < QueryRangeNum; QueryRangeIdx++)
		{
			if (QueryEntries[QueryRangeIdx]->bIsSelected)
			{
				SelectedRanges.Add(QueryRangeIdx);
			}
		}

		return bControlSetsUpdateRequired;
	}

	if (Database.IsNull() && (DatabaseContentHash != 0 || bForceRefresh))
	{
		bForceRefresh = false;
		DatabaseContentHash = 0;
		QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
		ControlSets.Reset();
		UpdateQueryEntries();
		SelectedRanges.Reset();
		return true;
	}

	return false;
}

#endif

UAnimGenController::UAnimGenController(const FObjectInitializer& ObjectInitializer)
{
#if WITH_EDITOR
	TrainingSettings = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimGenControllerTrainingSettings>(this, TEXT("TrainingSettings"));
	ViewportSettings = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimGenControllerViewportSettings>(this, TEXT("ViewportSettings"));
#endif
}

bool UAnimGenController::IsValid() const
{
	return 
		AutoEncoder &&
		AutoEncoder->IsValid() &&
		AutoEncoder->GetEncodingSize() == EncodedPoseMeans.Num() && 
		ControlSchema.IsValid() &&
		ControlSchema.IsElementValid(ControlSchemaElement) &&
		ControlEncoderNetwork && ControlEncoderNetwork->GetUpdateNumber() > 0 &&
		LOD0Network && LOD0Network->GetUpdateNumber() > 0 &&
		LOD1Network && LOD1Network->GetUpdateNumber() > 0 &&
		LOD2Network && LOD2Network->GetUpdateNumber() > 0;
}

void UAnimGenController::Invalidate()
{
	AutoEncoder = nullptr;
	ControlEncoderNetwork = nullptr;
	LOD0Network = nullptr;
	LOD1Network = nullptr;
	LOD2Network = nullptr;
	FrameRate = FFrameRate(60, 1);
	ControlVectorSize = 0;
	EncodedControlVectorSize = 0;
	ControlDistributionVectorSize = 0;

#if WITH_EDITOR
	TrainedFrameRangesContentHash = 0;
	TrainedAutoEncoderContentHash = 0;
	TrainedSchemaCompatibilityHash = 0;
	ControlEncoderSize = 0;
	DenoiserSize = 0;
#endif
}


void UAnimGenController::NormalizeEncodedPoseVectors(
	const TLearningArrayView<2, float> OutNormalizedEncodedPoseVectors,
	const TLearningArrayView<2, const float> InUnnormalizedEncodedPoseVectors) const
{
	const int32 FrameNum = OutNormalizedEncodedPoseVectors.Num<0>();
	const int32 DimNum = OutNormalizedEncodedPoseVectors.Num<1>();
	check(FrameNum == InUnnormalizedEncodedPoseVectors.Num<0>());
	check(DimNum == InUnnormalizedEncodedPoseVectors.Num<1>());
	check(DimNum == EncodedPoseMeans.Num());

	UE::Learning::Array::Copy(OutNormalizedEncodedPoseVectors, InUnnormalizedEncodedPoseVectors);
	UE::AnimDatabase::Math::NormalizeInplace(OutNormalizedEncodedPoseVectors, EncodedPoseMeans, EncodedPoseNormalizationScale);
}

void UAnimGenController::NormalizeEncodedPoseVectorsInplace(const TLearningArrayView<2, float> InOutEncodedPoseVectors) const
{
	const int32 FrameNum = InOutEncodedPoseVectors.Num<0>();
	const int32 DimNum = InOutEncodedPoseVectors.Num<1>();
	check(DimNum == EncodedPoseMeans.Num());

	UE::AnimDatabase::Math::NormalizeInplace(InOutEncodedPoseVectors, EncodedPoseMeans, EncodedPoseNormalizationScale);
}

void UAnimGenController::DenormalizeEncodedPoseVectors(
	const TLearningArrayView<2, float> OutUnnormalizedEncodedPoseVectors,
	const TLearningArrayView<2, const float> InNormalizedEncodedPoseVectors) const
{
	const int32 FrameNum = OutUnnormalizedEncodedPoseVectors.Num<0>();
	const int32 DimNum = OutUnnormalizedEncodedPoseVectors.Num<1>();
	check(FrameNum == InNormalizedEncodedPoseVectors.Num<0>());
	check(DimNum == InNormalizedEncodedPoseVectors.Num<1>());
	check(DimNum == EncodedPoseMeans.Num());

	UE::Learning::Array::Copy(OutUnnormalizedEncodedPoseVectors, InNormalizedEncodedPoseVectors);
	UE::AnimDatabase::Math::DenormalizeInplace(OutUnnormalizedEncodedPoseVectors, EncodedPoseMeans, EncodedPoseNormalizationScale);
}

void UAnimGenController::ClampNormalizedPoseVectorsInplace(
	const TLearningArrayView<2, float> InOutNormalizedEncodedPoseVectors) const
{
	const int32 DimNum = InOutNormalizedEncodedPoseVectors.Num<1>();
	check(DimNum == EncodedPoseMeans.Num());
	check(DimNum == EncodedPoseMins.Num());
	check(DimNum == EncodedPoseMaxs.Num());

	UE::AnimDatabase::Math::ClampNormalizedInplace(InOutNormalizedEncodedPoseVectors, EncodedPoseMeans, EncodedPoseNormalizationScale, EncodedPoseMins, EncodedPoseMaxs);
}

void UAnimGenController::ScaleSamplingNoiseInplace(const TLearningArrayView<2, float> InOutSamplingNoise) const
{
	const int32 DimNum = InOutSamplingNoise.Num<1>();
	check(DimNum == NormalizedPoseStds.Num());

	UE::AnimDatabase::Math::ScaleInplace(InOutSamplingNoise, NormalizedPoseStds);
}

#undef LOCTEXT_NAMESPACE
