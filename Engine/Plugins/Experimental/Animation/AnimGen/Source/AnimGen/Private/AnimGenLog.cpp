// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenLog.h"

DEFINE_LOG_CATEGORY(LogAnimGen);
