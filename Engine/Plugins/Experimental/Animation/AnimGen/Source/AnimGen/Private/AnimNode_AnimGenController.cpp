// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_AnimGenController.h"

#include "AnimGenAutoEncoder.h"
#include "AnimGenControl.h"
#include "AnimGenLog.h"
#include "AnimGenController.h"
#include "AnimGenBehavior.h"

#include "AnimDatabaseMath.h"
#include "AnimDatabasePose.h"

#include "LearningRandom.h"
#include "LearningObservation.h"
#include "LearningNeuralNetwork.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/BuiltInAttributeTypes.h"

#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseHistoryProvider.h"

#include "DrawDebugLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_AnimGenController)

#define LOCTEXT_NAMESPACE "AnimNode_AnimGenController"

void FAnimNode_AnimGenController::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);

	FAnimNode_Base::Initialize_AnyThread(Context);
	
	GetEvaluateGraphExposedInputs().Execute(Context);

	RandomState = Seed;
	RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
}

void FAnimNode_AnimGenController::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);

	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Update_AnyThread);

	FAnimNode_Base::Update_AnyThread(Context);

	const float DeltaTime = Context.GetDeltaTime();

	bPoseStateRequiresReset = bPoseStateRequiresReset || (bResetOnBecomingRelevant &&
		UpdateCounter.HasEverBeenUpdated() &&
		!UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()));

	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	GetEvaluateGraphExposedInputs().Execute(Context);

	// Update Active Controller

	if (!Controller || !Controller->IsValid()) { ActiveController = nullptr; return; }

	// Controller has changed so reset internal state

	if (Controller != ActiveController || 
		PoseVectorSize != ActiveController->AutoEncoder->GetPoseVectorSize() ||
		EncodedPoseVectorSize != ActiveController->AutoEncoder->GetEncodingSize() ||
		ControlSchemaHash != UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(
			ActiveController->ControlSchema.ObservationSchema->Schema, 
			ActiveController->ControlSchemaElement.SchemaElement) ||
		ControlVectorSize != ActiveController->ControlSchema.ObservationSchema->Schema.GetObservationVectorSize(ActiveController->ControlSchemaElement.SchemaElement) ||
		EncodedControlVectorSize != ActiveController->ControlSchema.ObservationSchema->Schema.GetEncodedVectorSize(ActiveController->ControlSchemaElement.SchemaElement) ||
		ControlDistributionVectorSize != ActiveController->ControlSchema.ObservationSchema->Schema.GetObservationDistributionVectorSize(ActiveController->ControlSchemaElement.SchemaElement))
	{
		// Update Active Controller

		ActiveController = Controller;

		// Get Control Schema

		const FAnimGenControlSchema& ControlSchema = ActiveController->ControlSchema;
		const FAnimGenControlSchemaElement& ControlSchemaElement = ActiveController->ControlSchemaElement;
		check(ControlSchema.ObservationSchema.IsValid() && ControlSchema.ObservationSchema->Schema.IsValid(ControlSchemaElement.SchemaElement));

		ControlSchemaHash = UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(ControlSchema.ObservationSchema->Schema, ControlSchemaElement.SchemaElement);
		ControlVectorSize = ControlSchema.ObservationSchema->Schema.GetObservationVectorSize(ControlSchemaElement.SchemaElement);
		EncodedControlVectorSize = ControlSchema.ObservationSchema->Schema.GetEncodedVectorSize(ControlSchemaElement.SchemaElement);
		ControlDistributionVectorSize = ControlSchema.ObservationSchema->Schema.GetObservationDistributionVectorSize(ControlSchemaElement.SchemaElement);

		ControlVector.SetNumUninitialized({ 1, ControlVectorSize });
		EncodedControlVector.SetNumUninitialized({ 1, EncodedControlVectorSize });
		UE::Learning::Array::Zero(ControlVector);
		UE::Learning::Array::Zero(EncodedControlVector);

		// Get Networks

		check(ActiveController->ControlEncoderNetwork);
		check(ActiveController->LOD0Network);
		check(ActiveController->LOD1Network);
		check(ActiveController->LOD2Network);

		ControlEncoderInference = Controller->ControlEncoderNetwork->GetNetwork()->CreateInferenceObject(1);
		LOD0Inference = Controller->LOD0Network->GetNetwork()->CreateInferenceObject(1);
		LOD1Inference = Controller->LOD1Network->GetNetwork()->CreateInferenceObject(1);
		LOD2Inference = Controller->LOD2Network->GetNetwork()->CreateInferenceObject(1);

		check(ActiveController->AutoEncoder);
		EncoderInference = ActiveController->AutoEncoder->EncoderNetwork->GetNetwork()->CreateInferenceObject(1);
		DecoderInference = ActiveController->AutoEncoder->DecoderNetwork->GetNetwork()->CreateInferenceObject(1);

		// Reset Pose State

		const int32 RequiredBoneNum = ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices.Num();

		PrevPoseData.Resize(1, RequiredBoneNum, ActiveController->AutoEncoder->GetAttributeTypes(), ActiveController->AutoEncoder->GetAttributeNames());
		CurrPoseData.Resize(1, RequiredBoneNum, ActiveController->AutoEncoder->GetAttributeTypes(), ActiveController->AutoEncoder->GetAttributeNames());
		OutputPoseData.Resize(1, RequiredBoneNum, ActiveController->AutoEncoder->GetAttributeTypes(), ActiveController->AutoEncoder->GetAttributeNames());

		ActiveController->AutoEncoder->SetDefaultPoseData(PrevPoseData.View(), ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices);
		ActiveController->AutoEncoder->SetDefaultPoseData(CurrPoseData.View(), ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices);
		ActiveController->AutoEncoder->SetDefaultPoseData(OutputPoseData.View(), ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices);

		// Compute Pose Vector

		PoseVectorSize = ActiveController->AutoEncoder->GetPoseVectorSize();
		EncodedPoseVectorSize = ActiveController->AutoEncoder->GetEncodingSize();

		PoseVector.SetNumUninitialized({ 1, PoseVectorSize });
		UnnormalizedEncodedPoseVector.SetNumUninitialized({ 1, EncodedPoseVectorSize });
		EncodedPoseVector.SetNumUninitialized({ 1, EncodedPoseVectorSize });
		EncodedPoseVectorBlock.SetNumUninitialized({ 4, EncodedPoseVectorSize });
		DenoiserInputVector.SetNumUninitialized({ 1, EncodedPoseVectorSize * 5 + EncodedControlVectorSize });
		UE::Learning::Array::Zero(PoseVector);
		UE::Learning::Array::Zero(UnnormalizedEncodedPoseVector);
		UE::Learning::Array::Zero(EncodedPoseVector);
		UE::Learning::Array::Zero(EncodedPoseVectorBlock);
		UE::Learning::Array::Zero(DenoiserInputVector);

		// Pose Needs Reset

		bPoseStateRequiresReset = true;
	}

	check(ActiveController && ActiveController->IsValid());

	// Check Controls are Valid

	const FAnimGenControlSchema& ControlSchema = ActiveController->ControlSchema;
	const FAnimGenControlSchemaElement& ControlSchemaElement = ActiveController->ControlSchemaElement;
	check(ControlSchema.IsValid() && ControlSchema.IsElementValid(ControlSchemaElement));

	if (!UAnimGenControls::ValidateControlObjectMatchesSchema(
		ControlSchema,
		ControlSchemaElement,
		ControlObject,
		ControlObjectElement))
	{
		if (ControlObject.IsValid())
		{
			UE::AnimGen::FSpinLockScope ObjectLock(ControlObject.ObservationObject->Lock);

			ControlObject.ObservationObject->Object.Reset();
		}

		bPoseStateRequiresReset = true;
		bControlsValid = false;
		return;
	}

	// Set Control Vector

	{
		UE::AnimGen::FSpinLockScope ObjectLock(ControlObject.ObservationObject->Lock);

		if (ControlObject.IsValid() && ControlObject.IsElementValid(ControlObjectElement))
		{
			UE::Learning::Observation::SetVectorFromObject(
				ControlVector[0],
				ControlSchema.ObservationSchema->Schema,
				ControlSchemaElement.SchemaElement,
				ControlObject.ObservationObject->Object,
				ControlObjectElement.ObjectElement);

			bControlsValid = true;
		}
		else
		{
			bPoseStateRequiresReset = true;
			bControlsValid = false;
			return;
		}
	}

	// Reset Control Object

	{
		UE::AnimGen::FSpinLockScope ObjectLock(ControlObject.ObservationObject->Lock);

		ControlObject.ObservationObject->Object.Reset();
	}

	// Clamp Control Vectors

	UE::Learning::Observation::ClampToObservationDistributionMinMax(
		ControlVector[0],
		ActiveController->ControlVectorDistributionMins,
		ActiveController->ControlVectorDistributionMaxs,
		ControlSchema.ObservationSchema->Schema,
		ControlSchemaElement.SchemaElement);

	// Reset Pose

	UAnimGenAutoEncoder* AutoEncoder = ActiveController->AutoEncoder;
	check(AutoEncoder&& AutoEncoder->IsValid());

	if (bPoseStateRequiresReset)
	{
		// Set to default

		AutoEncoder->SetDefaultPoseData(PrevPoseData.View(), AutoEncoder->AutoEncodedRequiredBoneIndices);
		AutoEncoder->SetDefaultPoseData(CurrPoseData.View(), AutoEncoder->AutoEncodedRequiredBoneIndices);

		// Reset Using Pose History

		const UE::PoseSearch::FPoseHistoryProvider* PoseHistoryProvider = Context.GetMessage<UE::PoseSearch::FPoseHistoryProvider>();
		const UE::PoseSearch::IPoseHistory* PoseHistory = PoseHistoryProvider ? PoseHistoryProvider->GetPoseHistoryPtr() : nullptr;

		if (PoseHistory)
		{
			// Reset Root

			FTransform CurrRootTransform = FTransform::Identity;

			if (PoseHistory->GetTransformAtTime(0.0f, CurrRootTransform, Context.AnimInstanceProxy->GetSkeleton(), 0, UE::PoseSearch::WorldSpaceIndexType))
			{
				CurrPoseData.RootData.RootLocations[0] = CurrRootTransform.GetLocation();
				CurrPoseData.RootData.RootRotations[0] = ((FQuat4f)CurrRootTransform.GetRotation()).GetNormalized();
				CurrPoseData.RootData.RootScales[0] = (FVector3f)CurrRootTransform.GetScale3D();

				FTransform PrevRootTransform = FTransform::Identity;

				if (DeltaTime > UE_SMALL_NUMBER &&
					PoseHistory->GetTransformAtTime(-DeltaTime, PrevRootTransform, Context.AnimInstanceProxy->GetSkeleton(), 0, UE::PoseSearch::WorldSpaceIndexType))
				{
					const FVector3f RootLinearVelocity = (FVector3f)(CurrRootTransform.GetLocation() - PrevRootTransform.GetLocation()) / DeltaTime;
					const FVector3f RootAngularVelocity = (FVector3f)(CurrRootTransform.GetRotation() * PrevRootTransform.GetRotation().Inverse()).GetShortestArcWith(FQuat::Identity).ToRotationVector() / DeltaTime;
					const FVector3f RootScalarVelocity = UE::AnimDatabase::Math::VectorLogSafe(UE::AnimDatabase::Math::VectorDivMax((FVector3f)CurrRootTransform.GetScale3D(), (FVector3f)PrevRootTransform.GetScale3D())) / DeltaTime;

					PrevPoseData.RootData.RootLocations[0] = PrevRootTransform.GetLocation();
					PrevPoseData.RootData.RootRotations[0] = ((FQuat4f)PrevRootTransform.GetRotation()).GetNormalized();
					PrevPoseData.RootData.RootScales[0] = (FVector3f)PrevRootTransform.GetScale3D();

					CurrPoseData.RootData.RootLinearVelocities[0] = RootLinearVelocity;
					CurrPoseData.RootData.RootAngularVelocities[0] = RootAngularVelocity;
					CurrPoseData.RootData.RootScalarVelocities[0] = RootScalarVelocity;

					PrevPoseData.RootData.RootLinearVelocities[0] = RootLinearVelocity;
					PrevPoseData.RootData.RootAngularVelocities[0] = RootAngularVelocity;
					PrevPoseData.RootData.RootScalarVelocities[0] = RootScalarVelocity;
				}
				else
				{
					PrevPoseData.RootData.RootLocations[0] = CurrRootTransform.GetLocation();
					PrevPoseData.RootData.RootRotations[0] = ((FQuat4f)CurrRootTransform.GetRotation()).GetNormalized();
					PrevPoseData.RootData.RootScales[0] = (FVector3f)CurrRootTransform.GetScale3D();
				}
			}
			else
			{
				const FTransform ComponentRootTransform = Context.AnimInstanceProxy->GetComponentTransform();

				CurrPoseData.RootData.RootLocations[0] = ComponentRootTransform.GetLocation();
				CurrPoseData.RootData.RootRotations[0] = ((FQuat4f)ComponentRootTransform.GetRotation()).GetNormalized();
				CurrPoseData.RootData.RootScales[0] = (FVector3f)ComponentRootTransform.GetScale3D();

				PrevPoseData.RootData.RootLocations[0] = ComponentRootTransform.GetLocation();
				PrevPoseData.RootData.RootRotations[0] = ((FQuat4f)ComponentRootTransform.GetRotation()).GetNormalized();
				PrevPoseData.RootData.RootScales[0] = (FVector3f)ComponentRootTransform.GetScale3D();
			}

			// Reset Bones

			const int32 RequiredBoneNum = AutoEncoder->AutoEncodedRequiredBoneIndices.Num();

			for (int32 RequiredBoneIdx = 0; RequiredBoneIdx < RequiredBoneNum; RequiredBoneIdx++)
			{
				const int32 AutoEncoderBoneIdx = AutoEncoder->AutoEncodedRequiredBoneIndices[RequiredBoneIdx];
				const int32 AutoEncoderParentIdx = ActiveController->AutoEncoder->GetBoneParent(AutoEncoderBoneIdx);
				if (AutoEncoderParentIdx == INDEX_NONE) { continue; }

				const int32 SkeletonBoneIdx = Context.AnimInstanceProxy->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(AutoEncoder->GetBoneName(AutoEncoderBoneIdx));
				const int32 SkeletonParentIdx = Context.AnimInstanceProxy->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(AutoEncoder->GetBoneName(AutoEncoderParentIdx));
				if (SkeletonBoneIdx == INDEX_NONE || SkeletonParentIdx == INDEX_NONE) { continue; }

				FTransform CurrBoneTransform = FTransform::Identity;
				FTransform CurrParentTransform = FTransform::Identity;

				if (PoseHistory->GetTransformAtTime(0.0f, CurrBoneTransform, Context.AnimInstanceProxy->GetSkeleton(), SkeletonBoneIdx) &&
					PoseHistory->GetTransformAtTime(0.0f, CurrParentTransform, Context.AnimInstanceProxy->GetSkeleton(), SkeletonParentIdx))
				{
					FTransform CurrTransform = CurrBoneTransform * CurrParentTransform.Inverse();

					CurrPoseData.LocalBoneData.BoneLocations[0][RequiredBoneIdx] = (FVector3f)CurrTransform.GetLocation();
					CurrPoseData.LocalBoneData.BoneRotations[0][RequiredBoneIdx] = (FQuat4f)CurrTransform.GetRotation();
					CurrPoseData.LocalBoneData.BoneScales[0][RequiredBoneIdx] = (FVector3f)CurrTransform.GetScale3D();

					FTransform PrevBoneTransform = FTransform::Identity;
					FTransform PrevParentTransform = FTransform::Identity;

					if (DeltaTime > UE_SMALL_NUMBER &&
						PoseHistory->GetTransformAtTime(-DeltaTime, PrevBoneTransform, Context.AnimInstanceProxy->GetSkeleton(), SkeletonBoneIdx) &&
						PoseHistory->GetTransformAtTime(-DeltaTime, PrevParentTransform, Context.AnimInstanceProxy->GetSkeleton(), SkeletonParentIdx))
					{
						FTransform PrevTransform = PrevBoneTransform * PrevParentTransform.Inverse();

						const FVector3f LinearVelocity = (FVector3f)(CurrTransform.GetLocation() - PrevTransform.GetLocation()) / DeltaTime;
						const FVector3f AngularVelocity = (FVector3f)(CurrTransform.GetRotation() * PrevTransform.GetRotation().Inverse()).GetShortestArcWith(FQuat::Identity).ToRotationVector() / DeltaTime;
						const FVector3f ScalarVelocity = UE::AnimDatabase::Math::VectorLogSafe(UE::AnimDatabase::Math::VectorDivMax((FVector3f)CurrTransform.GetScale3D(), (FVector3f)PrevTransform.GetScale3D())) / DeltaTime;

						PrevPoseData.LocalBoneData.BoneLocations[0][RequiredBoneIdx] = (FVector3f)PrevTransform.GetLocation();
						PrevPoseData.LocalBoneData.BoneRotations[0][RequiredBoneIdx] = (FQuat4f)PrevTransform.GetRotation();
						PrevPoseData.LocalBoneData.BoneScales[0][RequiredBoneIdx] = (FVector3f)PrevTransform.GetScale3D();

						CurrPoseData.LocalBoneData.BoneLinearVelocities[0][RequiredBoneIdx] = LinearVelocity;
						CurrPoseData.LocalBoneData.BoneAngularVelocities[0][RequiredBoneIdx] = AngularVelocity;
						CurrPoseData.LocalBoneData.BoneScalarVelocities[0][RequiredBoneIdx] = ScalarVelocity;

						PrevPoseData.LocalBoneData.BoneLinearVelocities[0][RequiredBoneIdx] = LinearVelocity;
						PrevPoseData.LocalBoneData.BoneAngularVelocities[0][RequiredBoneIdx] = AngularVelocity;
						PrevPoseData.LocalBoneData.BoneScalarVelocities[0][RequiredBoneIdx] = ScalarVelocity;
					}
					else
					{
						PrevPoseData.LocalBoneData.BoneLocations[0][RequiredBoneIdx] = (FVector3f)CurrTransform.GetLocation();
						PrevPoseData.LocalBoneData.BoneRotations[0][RequiredBoneIdx] = (FQuat4f)CurrTransform.GetRotation();
						PrevPoseData.LocalBoneData.BoneScales[0][RequiredBoneIdx] = (FVector3f)CurrTransform.GetScale3D();
					}
				}
			}
		}
		else
		{
			const FTransform ComponentRootTransform = Context.AnimInstanceProxy->GetComponentTransform();

			CurrPoseData.RootData.RootLocations[0] = ComponentRootTransform.GetLocation();
			CurrPoseData.RootData.RootRotations[0] = ((FQuat4f)ComponentRootTransform.GetRotation()).GetNormalized();
			CurrPoseData.RootData.RootScales[0] = (FVector3f)ComponentRootTransform.GetScale3D();

			PrevPoseData.RootData.RootLocations[0] = ComponentRootTransform.GetLocation();
			PrevPoseData.RootData.RootRotations[0] = ((FQuat4f)ComponentRootTransform.GetRotation()).GetNormalized();
			PrevPoseData.RootData.RootScales[0] = (FVector3f)ComponentRootTransform.GetScale3D();
		}

		AutoEncoder->ToPoseVectors(PoseVector, CurrPoseData.ConstView(), AutoEncoder->AutoEncodedRequiredBoneIndices);
		AutoEncoder->ClampPoseVectors(PoseVector);
		AutoEncoder->NormalizePoseVectors(PoseVector);

		// Encode Pose Vector

		if (EncoderInference->GetInputSize() == PoseVector.Num<1>() &&
			EncoderInference->GetOutputSize() == UnnormalizedEncodedPoseVector.Num<1>())
		{
			EncoderInference->Evaluate(UnnormalizedEncodedPoseVector, PoseVector);
		}
		else
		{
			UE::Learning::Array::Zero(UnnormalizedEncodedPoseVector);
		}

		// Normalize Encoded Pose Vector

		ActiveController->NormalizeEncodedPoseVectors(EncodedPoseVector, UnnormalizedEncodedPoseVector);
		ActiveController->ClampNormalizedPoseVectorsInplace(EncodedPoseVector);
	}

	// Evaluate Control Encoder

	if (ControlVector.Num<1>() == ControlEncoderInference->GetInputSize() &&
		EncodedControlVector.Num<1>() == ControlEncoderInference->GetOutputSize())
	{
		ControlEncoderInference->Evaluate(EncodedControlVector, ControlVector);
	}
	else
	{
		UE::Learning::Array::Zero(EncodedControlVector);
	}

	// Evaluate Denoiser

	if (DenoiserInputVector.Num() == LOD0Inference->GetInputSize() &&
		DenoiserInputVector.Num() == LOD1Inference->GetInputSize() &&
		DenoiserInputVector.Num() == LOD2Inference->GetInputSize() &&
		EncodedPoseVectorBlock.Num() == LOD0Inference->GetOutputSize() &&
		EncodedPoseVectorBlock.Num() == LOD1Inference->GetOutputSize() &&
		EncodedPoseVectorBlock.Num() == LOD2Inference->GetOutputSize())
	{
		if (bForceEvaluation || bPoseStateRequiresReset || EvaluationPeriodInFrames == 0.0f || (BlockTime + DeltaTime) * ActiveController->FrameRate.AsDecimal() >= EvaluationPeriodInFrames)
		{
			UE::Learning::Random::SampleClippedGaussianArray(EncodedPoseVectorBlock.Flatten(), RandomState, 0.0f, 1.0f, 3.0f);
			ActiveController->ScaleSamplingNoiseInplace(EncodedPoseVectorBlock);

			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(0 * EncodedPoseVectorSize, EncodedPoseVectorSize), EncodedPoseVector[0]);
			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(1 * EncodedPoseVectorSize, EncodedPoseVectorSize), EncodedPoseVectorBlock[0]);
			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(2 * EncodedPoseVectorSize, EncodedPoseVectorSize), EncodedPoseVectorBlock[1]);
			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(3 * EncodedPoseVectorSize, EncodedPoseVectorSize), EncodedPoseVectorBlock[2]);
			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(4 * EncodedPoseVectorSize, EncodedPoseVectorSize), EncodedPoseVectorBlock[3]);
			UE::Learning::Array::Copy(DenoiserInputVector[0].Slice(5 * EncodedPoseVectorSize, EncodedControlVectorSize), EncodedControlVector[0]);

			switch (LODLevel)
			{
			default:
			case 0: LOD0Inference->Evaluate(TLearningArrayView<2, float>(EncodedPoseVectorBlock.GetData(), { 1, 4 * EncodedPoseVectorSize }), DenoiserInputVector); break;
			case 1: LOD1Inference->Evaluate(TLearningArrayView<2, float>(EncodedPoseVectorBlock.GetData(), { 1, 4 * EncodedPoseVectorSize }), DenoiserInputVector); break;
			case 2: LOD2Inference->Evaluate(TLearningArrayView<2, float>(EncodedPoseVectorBlock.GetData(), { 1, 4 * EncodedPoseVectorSize }), DenoiserInputVector); break;
			}

			BlockTime = DeltaTime;
		}
		else
		{
			BlockTime += DeltaTime;
		}

		// Interpolate / Extrapolate using BlockTime

		const float PoseIndex = BlockTime * ActiveController->FrameRate.AsDecimal();

		if (PoseIndex <= 1.0f)
		{
			UE::AnimDatabase::Math::LerpToTargetInplace(
				EncodedPoseVector[0],
				EncodedPoseVectorBlock[0],
				FMath::Clamp(PoseIndex, 0.0f, 1.0f));
		}
		else if (PoseIndex > 1.0f && PoseIndex <= 2.0f)
		{
			UE::AnimDatabase::Math::ArrayInterpolateLinear(
				EncodedPoseVector[0],
				EncodedPoseVectorBlock[0],
				EncodedPoseVectorBlock[1],
				FMath::Clamp(PoseIndex - 1.0f, 0.0f, 1.0f));
		}
		else if (PoseIndex > 2.0f && PoseIndex <= 4.0f)
		{
			UE::AnimDatabase::Math::ArrayInterpolateLinear(
				EncodedPoseVector[0],
				EncodedPoseVectorBlock[1],
				EncodedPoseVectorBlock[2],
				FMath::Clamp((PoseIndex - 2.0f) / 2.0f, 0.0f, 1.0f));
		}
		else if (PoseIndex > 4.0f && PoseIndex <= 8.0f)
		{
			UE::AnimDatabase::Math::ArrayInterpolateLinear(
				EncodedPoseVector[0],
				EncodedPoseVectorBlock[2],
				EncodedPoseVectorBlock[3],
				FMath::Clamp((PoseIndex - 4.0f) / 4.0f, 0.0f, 1.0f));
		}
		else
		{
			UE::Learning::Array::Copy(EncodedPoseVector[0], EncodedPoseVectorBlock[3]);
		}

		// Clamp in-place

		ActiveController->ClampNormalizedPoseVectorsInplace(EncodedPoseVector);
	}

	// Evaluate Decoder

	ActiveController->DenormalizeEncodedPoseVectors(UnnormalizedEncodedPoseVector, EncodedPoseVector);

	if (DecoderInference->GetInputSize() == UnnormalizedEncodedPoseVector.Num<1>() &&
		DecoderInference->GetOutputSize() == PoseVector.Num<1>())
	{
		DecoderInference->Evaluate(PoseVector, UnnormalizedEncodedPoseVector);
	}
	else
	{
		UE::Learning::Array::Zero(PoseVector);
	}

	// Update Previous Pose

	PrevPoseData = CurrPoseData;

	// Update Pose

	const FTransform ComponentTransform = Context.AnimInstanceProxy->GetComponentTransform();

	AutoEncoder->DenormalizePoseVectors(PoseVector);
	AutoEncoder->ClampPoseVectors(PoseVector);
	AutoEncoder->FromPoseVectors(OutputPoseData.View(), PoseVector,
		{ ComponentTransform.GetLocation() },
		{ (FQuat4f)ComponentTransform.GetRotation() },
		ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices);

	CurrPoseData.RootData = OutputPoseData.RootData;

	if (bPoseStateRequiresReset)
	{
		CurrPoseData.LocalBoneData = OutputPoseData.LocalBoneData;
		CurrPoseData.AttributeData = OutputPoseData.AttributeData;
	}
	else
	{
		UE::AnimDatabase::PoseData::DampInplaceUsingAdaptiveVelocityIntegrationMix(
			CurrPoseData.LocalBoneData.View(),
			OutputPoseData.LocalBoneData.ConstView(),
			DeltaTime,
			MinPoseSmoothingTime,
			MaxPoseSmoothingTime);

		UE::AnimDatabase::PoseData::DampAttributesInPlace(
			CurrPoseData.AttributeData.View(),
			OutputPoseData.AttributeData.ConstView(),
			DeltaTime,
			AttributeSmoothingTime);
	}

	// Update Delta Time

	PoseDeltaTime = DeltaTime;

	// Write Anim Notifies

	AnimNotifyEvents.Reset();

	for (const FAnimGenControllerAnimNotifyOutput& AnimNotifyOutput : AnimNotifyOutputs)
	{
		if (!AnimNotifyOutput.AnimNotify) { continue; }
		if (AnimNotifyOutput.AutoEncoderAttributeName == NAME_None) { continue; }

		const int32 AttributeIdx = AutoEncoder->FindAttributeIndex(AnimNotifyOutput.AutoEncoderAttributeName);
		if (AttributeIdx == INDEX_NONE)
		{
			Context.LogMessage(EMessageSeverity::Warning, FText::Format(
				LOCTEXT("AttributeAnimNotifyMapAttributeNotFoundWarning", "Cannot create Anim Notify from channel '{0}' - not found in AutoEncoder."),
				FText::FromString(AnimNotifyOutput.AutoEncoderAttributeName.ToString())));
			continue;
		}

		if (AutoEncoder->GetAttributeType(AttributeIdx) != EAnimDatabaseAttributeType::Event)
		{
			Context.LogMessage(EMessageSeverity::Warning, FText::Format(
				LOCTEXT("AttributeAnimNotifyMapAttributeWrongTypeWarning", "Cannot create Anim Notify from channel '{0}' - channel is not Event type."),
				FText::FromString(AnimNotifyOutput.AutoEncoderAttributeName.ToString())));
			continue;
		}

		if (!PrevPoseData.AttributeData.GetAttributeActive(0, AttributeIdx) ||
			!CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx))
		{
			continue;
		}

		bool bPrevTimeUntilEventKnown = false;
		bool bCurrTimeUntilEventKnown = false;
		float PrevTimeUntilEvent = UE_MAX_FLT;
		float CurrTimeUntilEvent = UE_MAX_FLT;
		PrevPoseData.AttributeData.GetEvent(bPrevTimeUntilEventKnown, PrevTimeUntilEvent, 0, AttributeIdx);
		CurrPoseData.AttributeData.GetEvent(bCurrTimeUntilEventKnown, CurrTimeUntilEvent, 0, AttributeIdx);

		if (bPrevTimeUntilEventKnown &&
			bCurrTimeUntilEventKnown &&
			PrevTimeUntilEvent > AnimNotifyOutput.Apprehension &&
			CurrTimeUntilEvent < AnimNotifyOutput.Apprehension &&
			FMath::Abs(PrevTimeUntilEvent - CurrTimeUntilEvent) < 0.1f)
		{
			FAnimNotifyEvent& NotifyEvent = AnimNotifyEvents.AddDefaulted_GetRef();
			NotifyEvent.Notify = AnimNotifyOutput.AnimNotify;
			NotifyEvent.NotifyName = AnimNotifyOutput.AutoEncoderAttributeName;
		}
	}

	// Write Anim Notify States

	for (const FAnimGenControllerAnimNotifyStateOutput& AnimNotifyStateOutput : AnimNotifyStateOutputs)
	{
		if (!AnimNotifyStateOutput.AnimNotifyState) { continue; }
		if (AnimNotifyStateOutput.AutoEncoderAttributeName == NAME_None) { continue; }

		const int32 AttributeIdx = AutoEncoder->FindAttributeIndex(AnimNotifyStateOutput.AutoEncoderAttributeName);
		if (AttributeIdx == INDEX_NONE)
		{
			Context.LogMessage(EMessageSeverity::Warning, FText::Format(
				LOCTEXT("AttributeAnimNotifyStateMapAttributeNotFoundWarning", "Cannot create Anim Notify State from channel '{0}' - not found in AutoEncoder."),
				FText::FromString(AnimNotifyStateOutput.AutoEncoderAttributeName.ToString())));
			continue;
		}

		const EAnimDatabaseAttributeType Type = AutoEncoder->GetAttributeType(AttributeIdx);

		if (!AnimNotifyStateOutput.bOutputAutoEncoderAttributeActive && Type != EAnimDatabaseAttributeType::Bool && Type != EAnimDatabaseAttributeType::Float)
		{
			Context.LogMessage(EMessageSeverity::Warning, FText::Format(
				LOCTEXT("AttributeAnimNotifyStateMapAttributeWrongTypeWarning", "Cannot create Anim Notify State from channel '{0}' - channel is not Bool or Float type."),
				FText::FromString(AnimNotifyStateOutput.AutoEncoderAttributeName.ToString())));
			continue;
		}

		if (!CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx))
		{
			continue;
		}

		if (AnimNotifyStateOutput.bOutputAutoEncoderAttributeActive ||
			(Type == EAnimDatabaseAttributeType::Bool && CurrPoseData.AttributeData.GetBool(0, AttributeIdx)) ||
			(Type == EAnimDatabaseAttributeType::Float && CurrPoseData.AttributeData.GetFloat(0, AttributeIdx) != 0.0f))
		{
			FAnimNotifyEvent& NotifyEvent = AnimNotifyEvents.AddDefaulted_GetRef();
			NotifyEvent.NotifyStateClass = AnimNotifyStateOutput.AnimNotifyState;
			NotifyEvent.NotifyName = AnimNotifyStateOutput.AutoEncoderAttributeName;
		}
	}

	// Add Notifies and States to the AnimInstance

	AnimNotifyEventReferences.Reset();
	const int32 AnimNotifyEventNum = AnimNotifyEvents.Num();
	for (int32 AnimNotifyEventIdx = 0; AnimNotifyEventIdx < AnimNotifyEventNum; AnimNotifyEventIdx++)
	{
		FAnimNotifyEventReference& EventReference = AnimNotifyEventReferences.Emplace_GetRef(&AnimNotifyEvents[AnimNotifyEventIdx], Context.GetAnimInstanceObject());

		FAnimTickRecord TickRecord;
		TickRecord.bActiveContext = true;
		EventReference.GatherTickRecordData(TickRecord);
	}

	Context.AnimInstanceProxy->AddAnimNotifies(AnimNotifyEventReferences, 1.0f);

	// Reset of pose state done

	bPoseStateRequiresReset = false;
}

void FAnimNode_AnimGenController::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);

	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread);

	// Output Blank Pose

	Output.ResetToRefPose();

	if (!ActiveController || !bControlsValid) { return; }
	check(ActiveController->IsValid());

	const FBoneContainer& BoneContainer = Output.Pose.GetBoneContainer();
	TArrayView<FTransform> CompactBones = Output.Pose.GetMutableBones();
	const int32 CompactBoneNum = CompactBones.Num();

	// Output Compact Pose

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread::OutputPose);

		for (int32 BoneIdx = 0; BoneIdx < CompactBoneNum; BoneIdx++)
		{
			const int32 PoseBoneIdx = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(FCompactPoseBoneIndex(BoneIdx)).GetInt();
			if (PoseBoneIdx != INDEX_NONE)
			{
				const int32 MeshBoneIdx = BoneContainer.GetMeshPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(PoseBoneIdx)).GetInt();

				if (BoneContainer.GetReferenceSkeleton().IsValidIndex(MeshBoneIdx))
				{
					const int32 AutoEncoderBoneIdx = ActiveController->AutoEncoder->FindBoneIndex(BoneContainer.GetReferenceSkeleton().GetBoneName(MeshBoneIdx));

					if (AutoEncoderBoneIdx != INDEX_NONE)
					{
						const int32 RequiredBoneIdx = ActiveController->AutoEncoder->AutoEncodedRequiredBoneIndices.Find(AutoEncoderBoneIdx);

						if (RequiredBoneIdx != INDEX_NONE)
						{
							CompactBones[BoneIdx] = FTransform(
								((FQuat)CurrPoseData.LocalBoneData.BoneRotations[0][RequiredBoneIdx]).GetNormalized(),
								(FVector)CurrPoseData.LocalBoneData.BoneLocations[0][RequiredBoneIdx],
								(FVector)CurrPoseData.LocalBoneData.BoneScales[0][RequiredBoneIdx]);
						}
					}
				}
			}
		}
	}

	// Output additional bones

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread::OutputBones);

		for (const FAnimGenControllerBoneOutput& BoneOutput : BoneOutputs)
		{
			if (BoneOutput.AutoEncoderAttributeName == NAME_None || BoneOutput.OutputBoneName == NAME_None)
			{
				continue;
			}

			const int32 AttributeIdx = ActiveController->AutoEncoder->FindAttributeIndex(BoneOutput.AutoEncoderAttributeName);

			if (AttributeIdx == INDEX_NONE)
			{
				Output.LogMessage(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("AttributeTransformMapAttributeNotFoundWarning", "Cannot set bone from attribute '{0}' - not found in AutoEncoder."),
					FText::FromString(BoneOutput.AutoEncoderAttributeName.ToString())));
				continue;
			}

			const EAnimDatabaseAttributeType Type = ActiveController->AutoEncoder->GetAttributeType(AttributeIdx);

			if (Type != EAnimDatabaseAttributeType::Transform &&
				Type != EAnimDatabaseAttributeType::Location &&
				Type != EAnimDatabaseAttributeType::Rotation &&
				Type != EAnimDatabaseAttributeType::Scale)
			{
				Output.LogMessage(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("AttributeTransformMapAttributeWrongTypeWarning", "Cannot set bone from attribute '{0}' - attribute is not Transform, Location, Rotation, or Scale type."),
					FText::FromString(BoneOutput.AutoEncoderAttributeName.ToString())));
				continue;
			}

			if (CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx))
			{
				const int32 PoseBoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(BoneOutput.OutputBoneName);

				if (PoseBoneIndex != INDEX_NONE)
				{
					const FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(PoseBoneIndex));

					if (CompactBoneIndex.IsValid() && CompactBoneIndex >= 0 && CompactBoneIndex < CompactBones.Num())
					{
						switch (Type)
						{
						case EAnimDatabaseAttributeType::Transform:
						{
							const FTransform3f BoneTransform = CurrPoseData.AttributeData.GetTransform(0, AttributeIdx);

							CompactBones[CompactBoneIndex.GetInt()] = FTransform(
								((FQuat)BoneTransform.GetRotation()).GetNormalized(),
								(FVector)BoneTransform.GetLocation(),
								(FVector)BoneTransform.GetScale3D());
							break;
						}
						case EAnimDatabaseAttributeType::Location:
						{
							CompactBones[CompactBoneIndex.GetInt()].SetLocation((FVector)CurrPoseData.AttributeData.GetLocation(0, AttributeIdx));
							break;
						}
						case EAnimDatabaseAttributeType::Rotation:
						{
							CompactBones[CompactBoneIndex.GetInt()].SetRotation(((FQuat)CurrPoseData.AttributeData.GetRotation(0, AttributeIdx)).GetNormalized());
							break;
						}
						case EAnimDatabaseAttributeType::Scale:
						{
							CompactBones[CompactBoneIndex.GetInt()].SetScale3D((FVector)CurrPoseData.AttributeData.GetScale(0, AttributeIdx));
							break;
						}
						default:
						{
							checkNoEntry();
						}
						}

					}
				}
			}
		}
	}

	// Output Curves

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread::OutputCurves);

		OutputCurves.Reset();

		for (const FAnimGenControllerCurveOutput& CurveOutput : CurveOutputs)
		{
			if (CurveOutput.AutoEncoderAttributeName == NAME_None || CurveOutput.OutputCurveName == NAME_None)
			{
				continue;
			}

			const int32 AttributeIdx = ActiveController->AutoEncoder->FindAttributeIndex(CurveOutput.AutoEncoderAttributeName);

			if (AttributeIdx == INDEX_NONE)
			{
				Output.LogMessage(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("AttributeCurveMapAttributeNotFoundWarning", "Cannot create curve from attribute '{0}' - not found in AutoEncoder."),
					FText::FromString(CurveOutput.AutoEncoderAttributeName.ToString())));
				continue;
			}

			const EAnimDatabaseAttributeType Type = ActiveController->AutoEncoder->GetAttributeType(AttributeIdx);

			if (!CurveOutput.bOutputAutoEncoderAttributeActive &&
				Type != EAnimDatabaseAttributeType::Float &&
				Type != EAnimDatabaseAttributeType::Angle &&
				Type != EAnimDatabaseAttributeType::Bool)
			{
				Output.LogMessage(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("AttributeCurveMapAttributeWrongTypeWarning", "Cannot create curve from attribute '{0}' - attribute is not Float, Angle, or Bool type."),
					FText::FromString(CurveOutput.AutoEncoderAttributeName.ToString())));
				continue;
			}

			if (CurveOutput.bOutputAutoEncoderAttributeActive)
			{
				OutputCurves.Add({ CurveOutput.OutputCurveName, CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx) ? 1.0f : 0.0f });
			}
			else
			{
				if (CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx))
				{
					switch (Type)
					{
					case EAnimDatabaseAttributeType::Float: { OutputCurves.Add({ CurveOutput.OutputCurveName, CurrPoseData.AttributeData.GetFloat(0, AttributeIdx) }); break; }
					case EAnimDatabaseAttributeType::Bool: { OutputCurves.Add({ CurveOutput.OutputCurveName, CurrPoseData.AttributeData.GetBool(0, AttributeIdx) ? 1.0f : 0.0f }); break; }
					case EAnimDatabaseAttributeType::Angle: { OutputCurves.Add({ CurveOutput.OutputCurveName, FMath::RadiansToDegrees(CurrPoseData.AttributeData.GetAngle(0, AttributeIdx)) }); break; }
					default: { checkNoEntry(); }
					}
				}
			}
		}

		UE::Anim::FCurveUtils::BuildUnsorted(Output.Curve, OutputCurves);
	}

	// Output Additional Attributes

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread::OutputAttributes);

		for (const FAnimGenControllerAttributeOutput& AttributeOutput : AttributeOutputs)
		{
			if (AttributeOutput.AutoEncoderAttributeName == NAME_None ||
				AttributeOutput.OutputBoneName == NAME_None ||
				AttributeOutput.OutputAttributeName == NAME_None)
			{
				continue;
			}

			const int32 AttributeIdx = ActiveController->AutoEncoder->FindAttributeIndex(AttributeOutput.AutoEncoderAttributeName);

			if (AttributeIdx == INDEX_NONE)
			{
				Output.LogMessage(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("AttributeAttributeMapAttributeNotFoundWarning", "Cannot create attribute from attribute '{0}' - not found in AutoEncoder."),
					FText::FromString(AttributeOutput.AutoEncoderAttributeName.ToString())));
				continue;
			}

			const int32 PoseBoneIndex = BoneContainer.GetPoseBoneIndexForBoneName(AttributeOutput.OutputBoneName);

			if (PoseBoneIndex != INDEX_NONE)
			{
				const FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(PoseBoneIndex));

				if (CompactBoneIndex.IsValid() && CompactBoneIndex >= 0 && CompactBoneIndex < CompactBones.Num())
				{
					UE::Anim::FAttributeId AttributeId(AttributeOutput.OutputAttributeName, CompactBoneIndex);

					if (AttributeOutput.bOutputAutoEncoderAttributeActive)
					{
						FIntegerAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FIntegerAnimationAttribute>(AttributeId);
						Attr->Value = CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx) ? 1 : 0;
						break;
					}
					else if (CurrPoseData.AttributeData.GetAttributeActive(0, AttributeIdx))
					{
						const EAnimDatabaseAttributeType Type = ActiveController->AutoEncoder->GetAttributeType(AttributeIdx);

						switch (Type)
						{
						case EAnimDatabaseAttributeType::Null: { break; }
						case EAnimDatabaseAttributeType::Bool:
						{
							FIntegerAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FIntegerAnimationAttribute>(AttributeId);
							Attr->Value = CurrPoseData.AttributeData.GetBool(0, AttributeIdx) ? 1 : 0;
							break;
						}
						case EAnimDatabaseAttributeType::Float:
						{
							FFloatAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FFloatAnimationAttribute>(AttributeId);
							Attr->Value = CurrPoseData.AttributeData.GetFloat(0, AttributeIdx);
							break;
						}
						case EAnimDatabaseAttributeType::Location:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = (FVector)CurrPoseData.AttributeData.GetLocation(0, AttributeIdx);
							break;
						}
						case EAnimDatabaseAttributeType::Rotation:
						{
							FQuaternionAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FQuaternionAnimationAttribute>(AttributeId);
							Attr->Value = ((FQuat)CurrPoseData.AttributeData.GetRotation(0, AttributeIdx)).GetNormalized();
							break;
						}
						case EAnimDatabaseAttributeType::Scale:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = (FVector)CurrPoseData.AttributeData.GetScale(0, AttributeIdx);
							break;
						}
						case EAnimDatabaseAttributeType::LinearVelocity:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = (FVector)CurrPoseData.AttributeData.GetLinearVelocity(0, AttributeIdx);
							break;
						}
						case EAnimDatabaseAttributeType::AngularVelocity:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = (FVector)FMath::RadiansToDegrees(CurrPoseData.AttributeData.GetAngularVelocity(0, AttributeIdx));
							break;
						}
						case EAnimDatabaseAttributeType::ScalarVelocity:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = (FVector)CurrPoseData.AttributeData.GetScalarVelocity(0, AttributeIdx);
							break;
						}
						case EAnimDatabaseAttributeType::Direction:
						{
							FVectorAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FVectorAnimationAttribute>(AttributeId);
							Attr->Value = ((FVector)CurrPoseData.AttributeData.GetDirection(0, AttributeIdx)).GetSafeNormal();
							break;
						}
						case EAnimDatabaseAttributeType::Transform:
						{
							FTransformAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FTransformAnimationAttribute>(AttributeId);

							const FTransform3f Transform = CurrPoseData.AttributeData.GetTransform(0, AttributeIdx);

							Attr->Value = FTransform(
								((FQuat)Transform.GetRotation()).GetNormalized(),
								(FVector)Transform.GetLocation(),
								(FVector)Transform.GetScale3D());

							break;
						}
						case EAnimDatabaseAttributeType::Angle:
						{
							FFloatAnimationAttribute* Attr = Output.CustomAttributes.FindOrAdd<FFloatAnimationAttribute>(AttributeId);
							Attr->Value = FMath::RadiansToDegrees(CurrPoseData.AttributeData.GetAngle(0, AttributeIdx));
							break;
						}
						default: {}
						}
					}
				}
			}
		}
	}

	// Write Root Motion Attribute

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNode_AnimGenController::Evaluate_AnyThread::OutputRootMotion);

		if (RootMotionProvider)
		{
			FVector LocalRootLinearVelocity = (FVector)CurrPoseData.RootData.RootRotations[0].UnrotateVector(
				CurrPoseData.RootData.RootLinearVelocities[0]);

			FVector LocalRootAngularVelocity = (FVector)CurrPoseData.RootData.RootRotations[0].UnrotateVector(
				CurrPoseData.RootData.RootAngularVelocities[0]);

			if (bApplyRootVelocityDeadzone)
			{
				if (LocalRootLinearVelocity.SquaredLength() < FMath::Square(RootLinearVelocityDeadzone))
				{
					LocalRootLinearVelocity = FVector::ZeroVector;
				}

				if (LocalRootAngularVelocity.SquaredLength() < FMath::Square(FMath::DegreesToRadians(RootAngularVelocityDeadzone)))
				{
					LocalRootAngularVelocity = FVector::ZeroVector;
				}
			}

			const FTransform RootMotionDelta = FTransform(
				FQuat::MakeFromRotationVector(PoseDeltaTime * LocalRootAngularVelocity),
				PoseDeltaTime * LocalRootLinearVelocity);

			RootMotionProvider->SetRootMotion(RootMotionDelta, Output.CustomAttributes);
		}
	}
}

#undef LOCTEXT_NAMESPACE