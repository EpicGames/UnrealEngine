// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGenTraining.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameAttribute.h"

#include "LearningArray.h"

#include "Engine/DataAsset.h"
#include "Animation/BoneReference.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"

#include "AnimGenAutoEncoder.generated.h"

#define UE_API ANIMGEN_API

class USkeleton;
class USkeletalMesh;
class UAnimNotify;
class UAnimNotifyState;
class ULearningNeuralNetworkData;
class UAnimGenAutoEncoder;

namespace UE::AnimDatabase
{
	struct FPoseDataView;
	struct FPoseDataConstView;
}

/** Debug draw class used by the auto-encoder. Allows for custom debug-draw logic in the viewport */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimGenAutoEncoderDebugDraw : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	/** Make any required frame attributes or other data. This function is called once when the class is initialized */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimGen", meta = (ForceAsFunction))
	UE_API void InitializeDrawDebug(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const UAnimGenAutoEncoder* InAutoEncoder);

public:

	/** Draw Debug function which is called each frame with the current pose state information */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimGen", meta = (ForceAsFunction))
	UE_API void DrawDebug(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InOriginalPoseState,
		const FAnimDatabasePoseState& InReconstructedPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor,
		const FLinearColor& OriginalColor,
		const FLinearColor& ReconstructedColor);

#endif
};

/** Class containing all the UAnimGenAutoEncoder Editor window viewport settings */
UCLASS()
class UAnimGenAutoEncoderViewportSettings : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Mesh to use for previewing the character */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TObjectPtr<USkeletalMesh> PreviewMesh = nullptr;

	/** If to apply the character mesh to the reconstruction or to the original */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	bool bMeshOnReconstruction = true;

	/** Shows the character in the "default pose" used by the auto-encoder */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	bool bShowDefaultPose = false;

public:

	/** If to draw the original skeleton of all characters */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	bool bDrawOriginalSkeleton = true;

	/** The color to use for the drawing of the original skeleton */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton", HideEditConditionToggle))
	FLinearColor DrawOriginalSkeletonColor = FLinearColor(0.0f, 0.0f, 0.025f, 0.5f);

	/** If to draw the reconstructed skeleton of all characters */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	bool bDrawReconstructedSkeleton = true;

	/** The color to use for the reconstructed skeleton */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawReconstructedSkeleton", HideEditConditionToggle))
	FLinearColor DrawReconstructedSkeletonColor = FLinearColor(0.117f, 0.407f, 0.478f, 0.5f);

	/** If to only draw the bones that are being encoded by the auto-encoder */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	bool bDrawRequiredBonesOnly = true;

	/** Draw a simple skeleton made up of lines connecting bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	bool bDrawSimpleSkeleton = true;

	/** The scale of the drawn skeleton bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	float DrawSkeletonScale = 1.0f;

	/** If to draw linear velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	bool bDrawLinearVelocities = false;

	/** The scale of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "(bDrawOriginalSkeleton || bDrawReconstructedSkeleton) && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesScale = 0.05f;

	/** Opacity of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "(bDrawOriginalSkeleton || bDrawReconstructedSkeleton) && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesOpacity = 0.25f;

	/** If to draw angular velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	bool bDrawAngularVelocities = false;

	/** The scale of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "(bDrawOriginalSkeleton || bDrawReconstructedSkeleton) && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesScale = 2.5f;

	/** Opacity of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "(bDrawOriginalSkeleton || bDrawReconstructedSkeleton) && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesOpacity = 0.25f;

	/** If to draw bone transforms */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawOriginalSkeleton || bDrawReconstructedSkeleton", HideEditConditionToggle))
	bool bDrawBoneTransforms = false;

public:

	/** If to draw the root of the skeleton as a transform */
	UPROPERTY(EditAnywhere, Category = "Root")
	bool bDrawRoot = false;

	/** Opacity of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootOpacity = 1.0f;

	/** The scale of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootScale = 10.0f;

	/** If to integrate the root on the reconstructed pose rather than copy the root from the source */
	UPROPERTY(EditAnywhere, Category = "Root")
	bool bIntegrateRootOnReconstruction = false;

public:

	/** If to draw the trajectories of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	bool bDrawTrajectories = false;

	/** If to draw the trajectory orientations of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (EditCondition = "bDrawTrajectories", HideEditConditionToggle))
	bool bDrawTrajectoryOrientations = false;

public:

	/** Re-orients all ranges so that they start at the scene origin rather than wherever they started in the source data */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bRangesStartAtOrigin = true;

	/** Offset different ranges away from origin so they can be viewed together */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bOffsetRanges = true;

	/** Spacing to use for offsetting ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bOffsetRanges", HideEditConditionToggle))
	float RangeOffsetSpacing = 100.0f;

	/** If to offset selected ranges as a grid */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOffsetRanges", HideEditConditionToggle))
	bool bOffsetRangesInGrid = false;

	/** Remove root translation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bDisableRootTranslation = false;

	/** Remove root rotation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin && bDisableRootTranslation", HideEditConditionToggle))
	bool bDisableRootRotation = false;

	/** If to draw the colored range identifier over the character to show which range in the query the animation is from */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bDrawRangeIdentifier = true;

	/** Seed used for the colors of the range identifiers */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	int32 RangeIdentifierColorSeed = 1234;

	/** Thickness of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierThickness = 1.0f;

	/** Radius of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierRadius = 2.0f;

	/** Opacity of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierOpacity = 0.5f;

	/** Height offset used to draw the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierHeight = 200.0f;

	/** If to override the range identifier color */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bOverrideRangeIdentifierColor = false;

	/** Color to use to override the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOverrideRangeIdentifierColor", HideEditConditionToggle))
	FLinearColor RangeIdentifierColorOverride = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);

public:

	/** If to draw the scene origin as a transform */
	UPROPERTY(EditAnywhere, Category = "Origin")
	bool bDrawOrigin = true;

	/** Thickness of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginLineThickness = 1.0f;

	/** Opacity of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginOpacity = 0.5f;

	/** Scale of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginScale = 25.0f;

public:

#endif

#if WITH_EDITOR

	/** Rebuilds the encoding visualization */
	UFUNCTION(CallInEditor, Category = "Encoding Visualization", DisplayName="Rebuild Encoding Visualization")
	void RebuildVisualizationEncoding() { bForceRebuildEncodingVisualization = true; }

#endif

#if WITH_EDITORONLY_DATA

	bool bForceRebuildEncodingVisualization = false;

	/** Temp state controlled by the editor mode which says if the encoding visualization can be drawn */
	UPROPERTY(Transient)
	bool bCanDrawEncodingVisualization = false;

	/** If to draw the encoding visualization in the scene */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (EditCondition = "bCanDrawEncodingVisualization", HideEditConditionToggle))
	bool bDrawEncodingVisualization = true;

	/** If to draw the encoding visualization in a shared axis */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (EditCondition = "bCanDrawEncodingVisualization && bDrawEncodingVisualization", HideEditConditionToggle))
	bool bDrawSharedEncodingVisualization = false;

	/** Visualization encoding opacity */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bCanDrawEncodingVisualization && bDrawEncodingVisualization", HideEditConditionToggle))
	float DrawEncodingVisualizationOpacity = 0.5f;

	/** Visualization encoding line thickness */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bCanDrawEncodingVisualization && bDrawEncodingVisualization", HideEditConditionToggle))
	float DrawEncodingVisualizationThickness = 0.75f;

	/** Visualization encoding overall scale */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bCanDrawEncodingVisualization && bDrawEncodingVisualization", HideEditConditionToggle))
	float DrawEncodingVisualizationScale = 10.0f;

	/** Visualization encoding height offset */
	UPROPERTY(EditAnywhere, Category = "Encoding Visualization", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bCanDrawEncodingVisualization && bDrawEncodingVisualization", HideEditConditionToggle))
	float DrawEncodingVisualizationHeight = 300.0f;

#endif
};


/** Class containing all the UAnimGenAutoEncoder Editor window training settings */
UCLASS()
class UAnimGenAutoEncoderTrainingSettings : public UObject, public IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	/** Forces the Update function to refresh the internal state */
	UFUNCTION(CallInEditor, Category = "Data", DisplayName = "Refresh")
	UE_API void ForceRefresh();

	/** Load the current database. */
	UE_API void LoadDatabaseAsync();

	/** We use this async load changes to Database */
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& Event) override;

#endif

#if WITH_EDITORONLY_DATA

	/** Input database to train the network with */
	UPROPERTY(EditAnywhere, Category = "Data")
	TSoftObjectPtr<UAnimDatabase> Database = nullptr;

	/** Frame ranges in the database to use for training. Leave this as None to train on the full database */
	UPROPERTY(EditAnywhere, Instanced, Category = "Data")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges = nullptr;

public:

	/** Flag to force refresh the query during update */
	bool bForceRefresh = false;

	/** This hash is used to detect when the database might have changed and re-run the query */
	int32 DatabaseContentHash = 0;

	/** Query Ranges object for the current query */
	FAnimDatabaseFrameRanges QueryRanges;

	/** List of UI entries for the current query */
	TArray<TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>> QueryEntries;

	/** Indices of all the currently selected ranges */
	TArray<int32> SelectedRanges;

	/** Seed used for the colors of the range identifiers */
	int32 RangeIndentifierColorSeed = 1234;

#endif

#if WITH_EDITOR

	/**
	 * Checks if QueryRanges needs updating, and if so updates it. returns true if QueryRanges was updated (and therefore the UI may need updating),
	 * otherwise false.
	 */
	UE_API bool Update();

private:

	/** Internal function for updating the QueryEntries array from an updated QueryRanges */
	UE_API void UpdateQueryEntries();

#endif

#if WITH_EDITORONLY_DATA

public:

	/** List of bones to exclude from training */
	UPROPERTY(EditAnywhere, Category = "Data")
	TArray<FBoneReference> ExcludedBones;

	/** If to exclude virtual bones */
	UPROPERTY(EditAnywhere, Category = "Data", AdvancedDisplay)
	bool bExcludeVirtualBones = true;

	/** Increase this if the root motion is not being accurately reproduced */
	UPROPERTY(EditAnywhere, Category = "Data", AdvancedDisplay, meta = (ClampMin = "0.0", UIMin = "0.0"))
	float RootWeightMultiplier = 10.0f;

	/** Increase this if the network is not accurately reproducing bones at the end of joint chains */
	UPROPERTY(EditAnywhere, Category = "Data", AdvancedDisplay, meta = (ClampMin = "0.0", UIMin = "0.0"))
	float BaseWeightMultiplier = 1.0f;

	/** Increase this if the attributes are not being accurately reproduced */
	UPROPERTY(EditAnywhere, Category = "Data", AdvancedDisplay, meta = (ClampMin = "0.0", UIMin = "0.0"))
	float AttributeWeightMultiplier = 10.0f;

public:

	/** List of Frame Attributes to train the network to reproduce */
	UPROPERTY(EditAnywhere, Category = "Data")
	TArray<FAnimDatabaseFrameAttributeEntry> FrameAttributes;

public:

	/** The encoding size used by the auto-encoder to compress the pose  */
	UPROPERTY(EditAnywhere, Category = "Network", meta = (ClampMin = "1", UIMin = "1"))
	int32 EncodingSize = 150;

	/** The number of hidden units used by the auto-encoder on each layer */
	UPROPERTY(EditAnywhere, Category = "Network", meta = (ClampMin = "1", UIMin = "1"))
	int32 HiddenUnitNum = 512;

	/** The number of layers used by the encoder and decoder networks */
	UPROPERTY(EditAnywhere, Category = "Network", meta = (ClampMin = "1", UIMin = "1"))
	int32 LayerNum = 2;

public:

	/**
	 * If the auto-encoder should use compressed learning layers.
	 * This reduces the memory use by half at some very small cost to reproduction accuracy.
	 */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	bool bUseCompressedLinearLayers = true;

	/** The activation function used by the auto-encoder */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	EAnimGenActivationFunction ActivationFunction = EAnimGenActivationFunction::GELU;

	/** Network weight initialization method */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	EAnimGenWeightInit WeightInit = EAnimGenWeightInit::Uniform;

public:

	/**
	 * The number of iterations to train the auto-encoder for. Typical values are between 100000 and 2000000. Generally this does not need
	 * adjusting as 250000 is sufficient
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumberOfIterations = 250000;

	/** Learning rate. Typical values are between 0.001 and 0.0001. Reduce if training is unsable. Generally this does not need adjusting. */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRate = 0.001f;

	/**
	 * Batch size to use for training. Smaller values can sometimes produce better results at the cost of slowing down training. Large batch
	 * sizes are much more computationally efficient when training on the GPU. Reduce this if you are getting GPU out-of-memory errors during
	 * training. Generally this should not need adjusting.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "1", UIMin = "1", UIMax = "4096"))
	int32 BatchSize = 512;

	/**
	* If to enable logging to the UE log in the training subprocess. Enable this if training is failing for an unknown reason as it will allow you
	* to check the log for errors.
	*/
	UPROPERTY(EditAnywhere, Category = "Training")
	bool bEnableLogging = false;

	/** If to enable logging to TensorBoard in the intermediate directory. This can slow down training but gives important analytics. */
	UPROPERTY(EditAnywhere, Category = "Training")
	bool bEnableTensorboard = false;

public:

	/** The seed used for any random sampling the trainer will perform, e.g. for weight initialization. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "0", UIMin = "0"))
	int32 RandomSeed = 1234;

	/** The number of learning rate warm-up iterations to apply. Stabilizes the beginning of training. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 WarmupIterations = 1000;

	/** The device to train on. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay)
	EAnimGenTrainingDevice Device = EAnimGenTrainingDevice::GPU;

public:


	/** Custom database debug draw object that can be used to add additional debug draw behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Debug Draw")
	TObjectPtr<UAnimDatabaseDebugDraw> DatabaseDebugDrawer;

	/** Custom debug draw object that can be used to add additional debug draw behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Debug Draw")
	TObjectPtr<UAnimGenAutoEncoderDebugDraw> AutoEncoderDebugDrawer;

#endif

public: // IBoneReferenceSkeletonProvider
	USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;
};


/**
 * AnimGen AutoEncoder
 * 
 * This asset can be used to learn a compact vector encoding of poses (and their velocities) in conjunction with the AnimDatabase. This compact 
 * vector encoding is useful for downstream machine learning tasks since it allows them to work in an abstract space rather than dealing directly
 * with the pose representation.
 */
UCLASS(Blueprintable, BlueprintType)
class UAnimGenAutoEncoder : public UDataAsset
{
	GENERATED_BODY()

private:

	UE_API UAnimGenAutoEncoder(const FObjectInitializer& ObjectInitializer);

public:

	/** Check if the AutoEncoder is valid and trained */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API bool IsValid() const;

	/** Invalidate the AutoEncoder and reset back to a default state */
	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	UE_API void Invalidate();

	/** Gets the encoding size used by the AutoEncoder */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetEncodingSize() const;

	/** Gets the pose vector size used by the AutoEncoder */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetPoseVectorSize() const;

public:

	/** Gets the number of bones used by the AutoEncoder */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetBoneNum() const;

	/** Gets the bone name for a given AutoEncoder bone index */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API FName GetBoneName(const int32 BoneIdx) const;

	/** Gets the array of bone names for the AutoEncoder */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API const TArray<FName>& GetBoneNames() const;

	/** Finds the AutoEncoder bone index corresponding to a bone name */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 FindBoneIndex(const FName BoneName) const;

	/** Finds the AutoEncoder bone indices corresponding to the given bone names */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API void FindBoneIndices(TArray<int32>& OutBoneIndices, const TArray<FName>& BoneInBoneNamesNames) const;
	UE_API void FindBoneIndicesFromArrayViews(TArrayView<int32> OutBoneIndices, const TArrayView<const FName> InBoneNames) const;

	/** Gets the array of bone parent indices for the AutoEncoder */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API const TArray<int32>& GetBoneParents() const;

	/** Gets the parent for a given AutoEncoder bone index */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetBoneParent(const int32 BoneIdx) const;

public:

	/** Gets the number of attributes this AutoEncoder is trained on */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetAttributeNum() const;

	/** Gets the name of the attribute with the given index */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API FName GetAttributeName(const int32 AttributeIdx) const;

	/** Gets the type of the attribute with the given index */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API EAnimDatabaseAttributeType GetAttributeType(const int32 AttributeIdx) const;

	/** Finds the attribute index for a given name */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 FindAttributeIndex(const FName AttributeName) const;

	/** Gets an array of all of the trained attribute names */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API const TArray<FName>& GetAttributeNames() const;

	/** Gets an array of all of the trained attribute types **/
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API const TArray<EAnimDatabaseAttributeType>& GetAttributeTypes() const;

public:

	/**
	 * Gets a hash value which can be used to test if the underlying content of the auto-encoder networks has changed. This hash should
	 * not be relied on for detecting changes with certainty, and so should only be used for non-critical purposes (e.g. UI updates).
	 */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API int32 GetContentHash() const;

public:

	/** Converts the given pose data to pose vectors according to this AutoEncoder */
	UE_API void ToPoseVectors(
		const TLearningArrayView<2, float> OutPoseVectors,
		const UE::AnimDatabase::FPoseDataConstView& InPoseData) const;

	/** Converts the given pose data to pose vectors according to this AutoEncoder */
	UE_API void ToPoseVectors(
		const TLearningArrayView<2, float> OutPoseVectors,
		const UE::AnimDatabase::FPoseDataConstView& InPoseData,
		const UE::Learning::FIndexSet BoneIndices) const;

	/** Get the pose data from the given pose vectors according to this AutoEncoder */
	UE_API void FromPoseVectors(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const TLearningArrayView<2, const float> InPoseVectors,
		const TLearningArrayView<1, const FVector> InRootLocations,
		const TLearningArrayView<1, const FQuat4f> InRootRotations) const;

	/** Get the pose data from the given pose vectors according to this AutoEncoder */
	UE_API void FromPoseVectors(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const TLearningArrayView<2, const float> InPoseVectors,
		const TLearningArrayView<1, const FVector> InRootLocations,
		const TLearningArrayView<1, const FQuat4f> InRootRotations,
		const UE::Learning::FIndexSet BoneIndices) const;

	/** Sets the given pose data to the default pose of this AutoEncoder */
	UE_API void SetDefaultPoseData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData) const;

	/** Sets the given pose data to the default pose of this AutoEncoder */
	UE_API void SetDefaultPoseData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const UE::Learning::FIndexSet BoneIndices) const;

	/** Normalizes the given pose vectors according to this AutoEncoder */
	UE_API void NormalizePoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const;

	/** Denormalizes the given pose vectors according to this AutoEncoder */
	UE_API void DenormalizePoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const;

	/** Clamps the given pose vectors according to this AutoEncoder */
	UE_API void ClampPoseVectors(TLearningArrayView<2, float> InOutPoseVectors) const;

public:

	/** Encoder Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> EncoderNetwork = nullptr;

	/** Decoder Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> DecoderNetwork = nullptr;

public:

	/** Names of the bones used to train the auto-encoder */
	UPROPERTY()
	TArray<FName> BoneNames;

	/** Parents of the bones used to train the auto-encoder */
	UPROPERTY()
	TArray<int32> BoneParents;

	/** Default bone locations when this value is constant in the dataset */
	UPROPERTY()
	TArray<FVector3f> DefaultBoneLocations;
	
	/** Default bone rotations when this value is constant in the dataset */
	UPROPERTY()
	TArray<FQuat4f> DefaultBoneRotations;

	/** Default bone scales when this value is constant in the dataset */
	UPROPERTY()
	TArray<FVector3f> DefaultBoneScales;

	/** Indices of bone locations that are encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<int32> AutoEncodedBoneLocationIndices;

	/** Indices of bone rotations that are encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<int32> AutoEncodedBoneRotationIndices;

	/** Indices of bone scales that are encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<int32> AutoEncodedBoneScaleIndices;

	/** Indices of all of the bones encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<int32> AutoEncodedRequiredBoneIndices;

public:

	/** Normalization offset for pose vectors */
	UPROPERTY()
	TArray<float> PoseVectorOffset;

	/** Normalization scale for pose vectors */
	UPROPERTY()
	TArray<float> PoseVectorScale;

	/** Minimum pose vector for clamping */
	UPROPERTY()
	TArray<float> PoseVectorMin;

	/** Maximum pose vector for clamping */
	UPROPERTY()
	TArray<float> PoseVectorMax;

public:

	/** Names of the attributes encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<FName> AttributeNames;

	/** Types of the attributes encoded/decoded by this AutoEncoder */
	UPROPERTY()
	TArray<EAnimDatabaseAttributeType> AttributeTypes;

public:

#if WITH_EDITORONLY_DATA

	/** The skeleton used for training */
	UPROPERTY(VisibleAnywhere, Category = "Trained")
	TObjectPtr<USkeleton> TrainedSkeleton;

	/** The Frame Attributes used for training */
	UPROPERTY(VisibleAnywhere, Category = "Trained")
	TArray<FAnimDatabaseFrameAttributeEntry> TrainedFrameAttributes;

	/** List of bone names where the translation is output by the auto-encoder */
	UPROPERTY(VisibleAnywhere, Category = "Trained")
	TArray<FName> TrainedBoneLocations;

	/** List of bone names where the rotation is output by the auto-encoder */
	UPROPERTY(VisibleAnywhere, Category = "Trained")
	TArray<FName> TrainedBoneRotations;

	/** List of bone names where the scale is output by the auto-encoder */
	UPROPERTY(VisibleAnywhere, Category = "Trained")
	TArray<FName> TrainedBoneScales;

	/** The content hash of the frame ranges used for training */
	UPROPERTY()
	int32 TrainedContentHash = 0;

public:

	/** Current size of the trained encoder in kilobytes */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (ForceUnits = "Kilobytes"))
	int32 EncoderSize = 0;

	/** Current size of the trained decoder in kilobytes */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (ForceUnits = "Kilobytes"))
	int32 DecoderSize = 0;

	/** Current approximate inference time for the encoder in microseconds */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Statistics", meta = (ForceUnits = "Microseconds"))
	int32 EncoderInferenceTime = 0;

	/** Current approximate inference time for the decoder in microseconds */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Statistics", meta = (ForceUnits = "Microseconds"))
	int32 DecoderInferenceTime = 0;

public:

	/** Editor-only object containing the viewport settings for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimGenAutoEncoderViewportSettings> ViewportSettings;

	/** Editor-only object containing the training settings for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimGenAutoEncoderTrainingSettings> TrainingSettings;

#endif
};


/** Debug Draw class for drawing contact timings */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Timings Debug Draw"))
class UAnimGenAutoEncoderDebugDraw_ContactTimings : public UAnimGenAutoEncoderDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the left toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftToeBoneName = TEXT("ball_l");

	/** Name of the right toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightToeBoneName = TEXT("ball_r");

	/** Name of the left contact curve attribute */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftContactCurveAttribute = TEXT("contact_l");
	
	/** Name of the right contact curve attribute */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightContactCurveAttribute = TEXT("contact_r");

	/** Radius for the circle to display to show contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float ContactCircleRadius = 15.0f;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InOriginalPoseState,
		const FAnimDatabasePoseState& InReconstructedPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor,
		const FLinearColor& OriginalColor,
		const FLinearColor& ReconstructedColor) override;

#endif
};

/** Debug Draw class for drawing attributes and their reproductions */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Attributes"))
class UAnimGenAutoEncoderDebugDraw_Attributes : public UAnimGenAutoEncoderDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Spacing for text lines vertically */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VerticalSpacing = 16.0f;

	/** Pixel offset to display the properties */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float PropertyHorizontalOffset = 250.0f;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InOriginalPoseState,
		const FAnimDatabasePoseState& InReconstructedPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor,
		const FLinearColor& OriginalColor,
		const FLinearColor& ReconstructedColor) override;

#endif
};

#undef UE_API