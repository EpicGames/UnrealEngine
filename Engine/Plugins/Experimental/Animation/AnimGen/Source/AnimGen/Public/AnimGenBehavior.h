// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGenControl.h"

#include "Animation/TrajectoryTypes.h"
#include "InputCoreTypes.h"

#include "AnimGenBehavior.generated.h"

#define UE_API ANIMGEN_API

class UAnimGenController;
class UAnimGenAutoEncoder;

/**
 * A Behavior represents a specific control structure that can be used to control a character (for example - a trajectory to follow, a target point 
 * to move towards).
 * 
 * To define a behavior it is required to inherit from this class and to provide the "SpecifyControl", and "MakeControlSets"
 * functions. The "DrawDebugControl" function may also be provided to do debug drawing in the training viewport.
 */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimGenBehavior : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	/** Specify the control structure used by this behavior. */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "AnimGen", meta = (ForceAsFunction))
	UE_API FAnimGenControlSchemaElement SpecifyControl(UPARAM(ref) FAnimGenControlSchema& InControlSchema) const;

	/** Make the control sets used for training from the given database and frame ranges in the database */
	UFUNCTION(BlueprintNativeEvent, BlueprintPure, Category = "AnimGen", meta = (ForceAsFunction))
	UE_API void MakeControlSets(TArray<FAnimGenControlSet>& OutControlSets, UPARAM(ref) FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const;

	/** Draw debug the current frame in the training data */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimGen", meta = (ForceAsFunction))
	UE_API void DrawDebugControl(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const;

#endif
};

/** Behavior for uncontrolled generation. Produces a random-walk of the dataset. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Uncontrolled Behavior"))
class UAnimGenBehavior_Uncontrolled : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

#endif
};

/** Behavior for idling. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Idle Behavior"))
class UAnimGenBehavior_Idle : public UAnimGenBehavior_Uncontrolled
{
	GENERATED_BODY()
};

/** Behavior for following a trajectory which can either be generated from Mover/CharacterMovementComponent or a path to follow */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Trajectory Follow Behavior"))
class UAnimGenBehavior_TrajectoryFollow : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Future lookahead time for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float TrajectoryFutureTime = 1.0f;

	/** Past time for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float TrajectoryPastTime = 0.0f;

	/** Number of samples to use for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 TrajectorySampleNum = 4;

	/** Local root forward vector for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector TrajectoryForwardVector = FVector::RightVector;

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Behavior for following a velocity and angular velocity. More useful for offline generation. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Velocity Follow Behavior"))
class UAnimGenBehavior_VelocityFollow : public UAnimGenBehavior
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Behavior for moving toward a target with an optional facing direction */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Move to Target Behavior"))
class UAnimGenBehavior_MoveToTarget : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/**
 * Behavior for following a trajectory while also interacting with an object or prop. This is useful for networked prop interactions where the 
 * trajectory may be coming from a montage .
 */
UCLASS(BlueprintType, Blueprintable, DontCollapseCategories, meta = (DisplayName = "Trajectory Interaction Behavior"))
class UAnimGenBehavior_TrajectoryInteraction : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Future lookahead time for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	float TrajectoryFutureTime = 1.0f;

	/** Past time for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	float TrajectoryPastTime = 0.0f;

	/** Number of samples to use for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	int32 TrajectorySampleNum = 4;

	/** Local root forward vector for the trajectory */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	FVector TrajectoryForwardVector = FVector::RightVector;

	/** Name of bone used for interaction */
	UPROPERTY(EditAnywhere, Category = "Interaction")
	FName InteractionBoneName = TEXT("attach");

	/** Ranges where interaction is valid */
	UPROPERTY(EditAnywhere, Instanced, Category = "Interaction")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> InteractionRanges;

	/** Forward Vector for the interaction facing direction */
	UPROPERTY(EditAnywhere, Category = "Interaction")
	FVector InteractionForwardVector = FVector::ForwardVector;

	/** If to use a separate forward vector for mirrored data */
	UPROPERTY(EditAnywhere, Category = "Interaction")
	bool bUseMirroredInteractionForwardVector = false;

	/** Separate Forward Vector for the interaction facing direction when mirrored */
	UPROPERTY(EditAnywhere, Category = "Interaction", meta = (EditCondition = "bUseMirroredInteractionForwardVector", HideEditConditionToggle))
	FVector InteractionMirroredForwardVector = FVector(-1, 0, 0);

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Behavior that allows a user to supply tags to control the style of a sub-behavior. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Tagged Behavior"))
class UAnimGenBehavior_Tagged : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Sub-behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimGenBehavior> Behavior;

	/** List of tags with associated frame ranges */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimDatabaseFrameRangesEntry> TagRanges;

	/** Offset to debug draw the current tags */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector TagDebugDrawOffset = FVector(0.0, 0.0, 220.0f);

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Behavior that allows a user to optionally provide controls to another Behavior. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Optional Behavior"))
class UAnimGenBehavior_Optional : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Sub-behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimGenBehavior> Behavior;

	/** Size to encode the sub-behaviors controls to */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (UIMin = "1", ClampMin = "1"))
	int32 EncodingSize = 128;

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Behavior that encodes the controls of some sub-behavior. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Encoded Behavior"))
class UAnimGenBehavior_Encoded : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Sub-behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimGenBehavior> Behavior;

	/** Size to encode the sub-behaviors controls to */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (UIMin = "1", ClampMin = "1"))
	int32 EncodingSize = 128;
	
	/** Number of layers to use to encode the sub-behavior controls */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (UIMin = "1", ClampMin = "1"))
	int32 HiddenLayerNum = 1;
	
	/** Activation function to use to encode the sub-behavior controls */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimGenActivationFunction ActivationFunction = EAnimGenActivationFunction::ELU;

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Entry specifying a behavior */
USTRUCT(BlueprintType)
struct FAnimGenBehaviorEntry
{
	GENERATED_BODY()

public:

	/** Behavior name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName Name = NAME_None;

	/** Behavior object */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimGenBehavior> Behavior;

	/** Frame ranges to use for behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;
};

/** Behavior made up of multiple other behaviors - user can also provide some tags to control the style. */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Multi Behavior"))
class UAnimGenBehavior_Multi : public UAnimGenBehavior
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Different potential sub-behaviors */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimGenBehaviorEntry> Behaviors;

#endif

#if WITH_EDITOR

public:

	UE_API virtual FAnimGenControlSchemaElement SpecifyControl_Implementation(FAnimGenControlSchema& InControlSchema) const override;

	UE_API virtual void MakeControlSets_Implementation(TArray<FAnimGenControlSet>& OutControlSets, FAnimGenControlObject& InControlObject, const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) const override;

	UE_API virtual void DrawDebugControl_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement InControlObjectElement,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const UAnimGenAutoEncoder* InAutoEncoder,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) const override;

#endif
};

/** Blueprint library of helper functions for making and testing behaviors including functions for preparing, processing, and making controls */
UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimGenBehaviorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Make a control object element for the Uncontrolled Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeUncontrolledBehaviorControl(UPARAM(ref) FAnimGenControlObject& InControlObject);

	/** Make a control object element for the Idle Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeIdleBehaviorControl(UPARAM(ref) FAnimGenControlObject& InControlObject);

	/** Make a control object element for the Trajectory Following Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AutoCreateRefTerm = "TransformTrajectory, RootBoneTransform"))
	static UE_API FAnimGenControlObjectElement MakeTrajectoryFollowBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FTransformTrajectory& TransformTrajectory,
		const FTransform& RootBoneTransform,
		const int32 InTrajectorySampleNum = 4,
		const float InTrajectoryFutureTime = 1.0f,
		const float InTrajectoryPastTime = 0.0f,
		const FVector InTrajectoryForwardVector = FVector::RightVector);

	/** Make a control object element for the Move To Target Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AutoCreateRefTerm = "RootBoneTransform"))
	static UE_API FAnimGenControlObjectElement MakeMoveToTargetBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FVector TargetLocation,
		const FVector TargetDirection,
		const bool bUseTargetDirection,
		const FTransform& RootBoneTransform);

	/** Make a control object element for the Trajectory Interaction Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AutoCreateRefTerm = "TransformTrajectory, RootBoneTransform"))
	static UE_API FAnimGenControlObjectElement MakeTrajectoryInteractionBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FTransformTrajectory& TransformTrajectory,
		const FTransform& RootBoneTransform,
		const FVector InteractionLocation = FVector::ZeroVector,
		const FVector InteractionDirection = FVector::ForwardVector,
		const bool bInteractionActive = false,
		const int32 InTrajectorySampleNum = 4,
		const float InTrajectoryFutureTime = 1.0f,
		const float InTrajectoryPastTime = 0.0f,
		const FVector InTrajectoryForwardVector = FVector::RightVector);

	/** Make a control object element for the Tagged Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AutoCreateRefTerm = "DesiredTags"))
	static UE_API FAnimGenControlObjectElement MakeTaggedBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement& BehaviorControlElement,
		const TArray<FName>& DesiredTags);

	static UE_API FAnimGenControlObjectElement MakeTaggedBehaviorControlFromArrayView(
		FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement& BehaviorControlElement,
		const TArrayView<const FName> DesiredTags);

	/** Make a control object element for the Multi Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeMultiBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FName Behavior,
		const FAnimGenControlObjectElement& BehaviorControlElement);

	/** Make a valid control object element for the Optional Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeValidOptionalBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement& BehaviorControlElement);

	/** Make a null control object element for the Optional Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeNullOptionalBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject);

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeValidOptionalBehaviorControlOnCondition(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement& BehaviorControlElement,
		const bool bCondition);

	/** Make a control object element for the Encoded Behavior */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlObjectElement MakeEncodedBehaviorControl(
		UPARAM(ref) FAnimGenControlObject& InControlObject,
		const FAnimGenControlObjectElement& BehaviorControlElement);

public:

	/** Projects a location onto the floor plane given by the provided height */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FVector ProjectOntoFloor(const FVector Location, const float FloorHeight = 0.0f);

	/** Clamps one vector to some maximum distance from another vector */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FVector ClampToMaxDistanceFromVector(
		const FVector InVector,
		const FVector FromVector,
		const float MaxDistance = 100.0f);

	/** Apply proper dead-zone and sensitivity treatment to stick input */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API void ApplyStickDeadzoneAndSensitivity(
		FVector& OutAxis,
		FVector& OutDirection,
		float& OutLength,
		bool& bOutNotInDeadzone,
		const FVector Axis,
		const float Deadzone = 0.2f,
		const float Sensitivity = 2.0f);

	/** Apply sensitivity to a trigger value */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API float ApplyTriggerSensitivity(const float Trigger, const float Sensitivity = 2.0f);

	/** Check if the stick is not in the deadzone */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool IsStickNotInDeadzone(const FVector Axis, const float Deadzone = 0.2f);

	/** Check if the stick is in the deadzone */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool IsStickInDeadzone(const FVector Axis, const float Deadzone = 0.2f);

	/** Convert a desired direction to a desired rotation */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FRotator DesiredDirectionToRotation(const FVector Direction);

	/** Convert a desired direction to a desired rotation quaternion */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FQuat DesiredDirectionToRotationQuat(const FVector Direction);

	/** Update the desired velocity from a stick axis */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API void UpdateDesiredVelocityFromAxis(
		UPARAM(ref) FVector& DesiredVelocity,
		const FVector Axis,
		const float VelocitySpeed = 150.0f,
		const float Deadzone = 0.2f,
		const float Sensitivity = 2.0f);

	/** Update the desired rotation from a stick axis */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API void UpdateDesiredRotationFromAxis(
		UPARAM(ref) FRotator& DesiredRotation,
		const FVector Axis,
		const float Deadzone = 0.2f,
		const float Sensitivity = 2.0f);

	/** Update the desired velocity and rotation from two stick axes */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API void UpdateDesiredVelocityAndRotationFromAxes(
		UPARAM(ref) FVector& DesiredVelocity,
		UPARAM(ref) FRotator& DesiredRotation,
		const FVector VelocityAxis,
		const FVector FacingAxis,
		const float VelocitySpeed = 150.0f,
		const float Deadzone = 0.2f,
		const float Sensitivity = 2.0f);

	/** Update the desired velocity and rotation from a stick axes and camera trigger */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API void UpdateDesiredVelocityAndRotationFromAxisAndCameraTrigger(
		UPARAM(ref) FVector& DesiredVelocity,
		UPARAM(ref) FRotator& DesiredRotation,
		const FRotator CameraYawRotation,
		const FVector VelocityAxis,
		const float FacingTrigger,
		const float MinimumSpeed = 75.0f,
		const float MaximumSpeed = 150.0f,
		const float Deadzone = 0.2f,
		const float Sensitivity = 2.0f);

	/** Find the index of the most similar value in an array of floats */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API int32 FindNearestIndexInFloatArray(const TArray<float>& Values, const float Value);

	/** Cycle through an array of names and return the current name */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API FName CycleNameInArray(const TArray<FName>& Names, const FName CurrentName, const int32 Steps = 1);
	static UE_API FName CycleNameInArrayView(const TArrayView<const FName> Names, const FName CurrentName, const int32 Steps = 1);

	/** Check if two locations are within some distance threshold */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool LocationDifferenceBelowThreshold(
		const FVector CurrentLocation,
		const FVector TargetLocation,
		const float LocationThreshold = 25.0f);

	/** Check if locations and directions are within some distance and angle threshold (in degrees) */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool LocationAndDirectionDifferenceBelowThreshold(
		const FVector CurrentLocation,
		const FVector CurrentDirection,
		const FVector TargetLocation,
		const FVector TargetDirection,
		const float LocationThreshold = 25.0f,
		const float DirectionThresoldAngle = 10.0f);

public:

	/** Applies a simple avoidance heuristic to a movement velocity */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API FVector ApplySimpleAvoidanceToVelocity(
		const FVector Velocity,
		const TArray<AActor*>& Actors,
		const AActor* SelfActor,
		const float MinAvoidanceRadius = 25.0f,
		const float MaxAvoidanceRadius = 150.0f,
		const float AvoidanceVelocity = 200.0f);

	static UE_API FVector ApplySimpleAvoidanceToVelocityArrayView(
		const FVector Velocity,
		const TArrayView<AActor* const> Actors,
		const AActor* SelfActor,
		const float MinAvoidanceRadius = 25.0f,
		const float MaxAvoidanceRadius = 150.0f,
		const float AvoidanceVelocity = 200.0f);

	/** Checks if an actor can look at another actor according to some distance and angle thresholds */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API bool CanLookAtActor(
		const FVector LookAtDirection,
		const AActor* Actor,
		const AActor* SelfActor,
		const float DistanceThreshold = 200.0f,
		const float DotAngleThreshold = 0.75f);

	/** Finds an actor to look-at according to some distance and angle thresholds */
	UFUNCTION(BlueprintPure = false, Category = "AnimGen")
	static UE_API AActor* FindLookAtActor(
		const FVector LookAtDirection,
		const TArray<AActor*>& Actors,
		const AActor* SelfActor,
		const float DistanceThreshold = 200.0f,
		const float DotAngleThreshold = 0.75f);

	static UE_API AActor* FindLookAtActorArrayView(
		const FVector LookAtDirection,
		const TArray<AActor*>& Actors,
		const AActor* SelfActor,
		const float DistanceThreshold = 200.0f,
		const float DotAngleThreshold = 0.75f);
};

#undef UE_API