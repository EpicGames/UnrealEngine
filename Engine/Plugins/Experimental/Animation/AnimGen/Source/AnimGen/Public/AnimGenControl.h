// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGenTraining.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameRanges.h"

#include "LearningObservation.h"
#include "LearningFrameRangeSet.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/SpinLock.h"

#include "AnimGenControl.generated.h"

#define UE_API ANIMGEN_API

struct FTransformTrajectory;

class UAnimDatabase;
class UAnimGenAutoEncoder;

/** Blueprint accessible version of a control schema element. Represents a single control type in the schema. */
USTRUCT(BlueprintType)
struct FAnimGenControlSchemaElement
{
	GENERATED_BODY()

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	UE::Learning::Observation::FSchemaElement SchemaElement;
};

UE_API bool operator==(const FAnimGenControlSchemaElement& Lhs, const FAnimGenControlSchemaElement& Rhs);

UE_API uint32 GetTypeHash(const FAnimGenControlSchemaElement& Element);

template<>
struct TStructOpsTypeTraits<FAnimGenControlSchemaElement> : public TStructOpsTypeTraitsBase2<FAnimGenControlSchemaElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

/** Blueprint accessible version of a control object element. Represents a single instance of a control object type. */
USTRUCT(BlueprintType)
struct FAnimGenControlObjectElement
{
	GENERATED_BODY()

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	UE::Learning::Observation::FObjectElement ObjectElement;
};

UE_API bool operator==(const FAnimGenControlObjectElement& Lhs, const FAnimGenControlObjectElement& Rhs);

UE_API uint32 GetTypeHash(const FAnimGenControlObjectElement& Element);

template<>
struct TStructOpsTypeTraits<FAnimGenControlObjectElement> : public TStructOpsTypeTraitsBase2<FAnimGenControlObjectElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

namespace UE::AnimGen
{
	/** Wrapped FSchema object to include a lock to allow for BlueprintThreadSafe access */
	struct FControlSchema
	{
		FSpinLock Lock;
		Learning::Observation::FSchema Schema;
	};

	/** Wrapped FObject object to include a lock to allow for BlueprintThreadSafe access */
	struct FControlObject
	{
		FSpinLock Lock;
		Learning::Observation::FObject Object;
	};

	class FSpinLockScope
	{
	public:
		UE_NODISCARD_CTOR explicit FSpinLockScope(FSpinLock& InLock)
			: Lock(InLock)
		{
			Lock.Lock();
		}

		~FSpinLockScope()
		{
			Lock.Unlock();
		}

	private:
		FSpinLock& Lock;

		UE_NONCOPYABLE(FSpinLockScope);
	};
}

/** Blueprint accessible version of a control schema */
USTRUCT(BlueprintType)
struct FAnimGenControlSchema
{
	GENERATED_BODY()
	
	/** Check if the schema is valid */
	UE_API bool IsValid() const;

	/** Check if the given element is valid for this schema */
	UE_API bool IsElementValid(const FAnimGenControlSchemaElement& Element) const;

	/** Invalidate the schema */
	UE_API void Invalidate();

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	TSharedPtr<UE::AnimGen::FControlSchema, ESPMode::ThreadSafe> ObservationSchema;
};

UE_API bool operator==(const FAnimGenControlSchema& Lhs, const FAnimGenControlSchema& Rhs);

template<>
struct TStructOpsTypeTraits<FAnimGenControlSchema> : public TStructOpsTypeTraitsBase2<FAnimGenControlSchema>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

/** Blueprint accessible version of a control object */
USTRUCT(BlueprintType)
struct FAnimGenControlObject
{
	GENERATED_BODY()

	/** Check if the control object is valid */
	UE_API bool IsValid() const;

	/** Check if the given element is valid for this control object */
	UE_API bool IsElementValid(const FAnimGenControlObjectElement& Element) const;

	/** Invalidate the control object */
	UE_API void Invalidate();

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	TSharedPtr<UE::AnimGen::FControlObject, ESPMode::ThreadSafe> ObservationObject;
};

UE_API bool operator==(const FAnimGenControlObject& Lhs, const FAnimGenControlObject& Rhs);

template<>
struct TStructOpsTypeTraits<FAnimGenControlObject> : public TStructOpsTypeTraitsBase2<FAnimGenControlObject>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

/** Enum used to determine between two options in an Either control */
UENUM(BlueprintType)
enum class EAnimGenEitherControl : uint8
{
	A,
	B,
};

/** Enum used to determine between null and valid in an Optional control */
UENUM(BlueprintType)
enum class EAnimGenOptionalControl : uint8
{
	Null,
	Valid,
};

namespace UE::AnimGen
{
	/** Control set data contains the frame ranges in the database associated with the controls and each control element */
	struct FControlSetData
	{
		UE::Learning::FFrameRangeSet FrameRangeSet;
		TArray<FAnimGenControlObjectElement> Controls;
	};
}

/** Blueprint accessible version of a control set. Represents a set of controls for a set of frames in the database */
USTRUCT(BlueprintType)
struct FAnimGenControlSet
{
	GENERATED_BODY()

	/** Get the array of control object elements for this control set */
	UE_API TArrayView<const FAnimGenControlObjectElement> GetControls() const;

	/** Check if this control set is valid */
	UE_API bool IsValid() const;

	/*** Invalidate this control set */
	UE_API void Invalidate();

	TSharedPtr<UE::AnimGen::FControlSetData, ESPMode::ThreadSafe> Data;
};

/** Blueprint function library used for creating control schema elements, control object elements, and control sets */
UCLASS(BlueprintType, meta=(BlueprintThreadSafe))
class UAnimGenControls : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	static UE_API FAnimGenControlSchema MakeControlSchema();

	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	static UE_API FAnimGenControlObject MakeControlObject();

	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	static UE_API void ResetControlSchema(UPARAM(ref) FAnimGenControlSchema& Schema);

	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	static UE_API void ResetControlObject(UPARAM(ref) FAnimGenControlObject& Object);

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool IsControlSchemaValid(const FAnimGenControlSchema& Schema);

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool IsControlObjectValid(const FAnimGenControlObject& Object);

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API bool ValidateControlObjectMatchesSchema(
		const FAnimGenControlSchema& Schema,
		const FAnimGenControlSchemaElement SchemaElement,
		const FAnimGenControlObject& Object,
		const FAnimGenControlObjectElement ObjectElement);

public:

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyNullControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("NullControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyContinuousControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Size, const FName Tag = TEXT("ContinuousControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyNamedExclusiveDiscreteControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));
	static UE_API FAnimGenControlSchemaElement SpecifyNamedExclusiveDiscreteControlFromArrayView(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyNamedInclusiveDiscreteControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));
	static UE_API FAnimGenControlSchemaElement SpecifyNamedInclusiveDiscreteControlFromArrayView(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyExclusiveDiscreteControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Size, const FName Tag = TEXT("ExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyInclusiveDiscreteControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Size, const FName Tag = TEXT("InclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyCountControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("CountControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyStructControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyStructControlFromArrays(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const FName Tag = TEXT("StructControl"));
	static UE_API FAnimGenControlSchemaElement SpecifyStructControlFromArrayViews(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyExclusiveUnionControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const int32 EncodingSize = 128, const FName Tag = TEXT("ExclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyExclusiveUnionControlFromArrays(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const int32 EncodingSize = 128, const FName Tag = TEXT("ExclusiveUnionControl"));

	static UE_API FAnimGenControlSchemaElement SpecifyExclusiveUnionControlFromArrayViews(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const int32 EncodingSize = 128, const FName Tag = TEXT("ExclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyInclusiveUnionControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TMap<FName, FAnimGenControlSchemaElement>& Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyInclusiveUnionControlFromArrays(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const TArray<FAnimGenControlSchemaElement>& Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("InclusiveUnionControl"));
	static UE_API FAnimGenControlSchemaElement SpecifyInclusiveUnionControlFromArrayViews(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlSchemaElement> Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyStaticArrayControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 Num, const FName Tag = TEXT("StaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifySetControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyNamedSparseControl(UPARAM(ref) FAnimGenControlSchema& Schema, const TArray<FName>& ElementNames, const FName Tag = TEXT("NamedSparseControl"));
	static UE_API FAnimGenControlSchemaElement SpecifyNamedSparseControlFromArrayView(UPARAM(ref) FAnimGenControlSchema& Schema, const TArrayView<const FName> ElementNames, const FName Tag = TEXT("NamedSparseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifySparseControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Size, const FName Tag = TEXT("SparseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyPairControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Key, const FAnimGenControlSchemaElement Value, const FName Tag = TEXT("PairControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyArrayControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("ArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSchemaElement SpecifyMapControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement KeyElement, const FAnimGenControlSchemaElement ValueElement, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyEnumControl(UPARAM(ref) FAnimGenControlSchema& Schema, const UEnum* Enum, const FName Tag = TEXT("EnumControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyBitmaskControl(UPARAM(ref) FAnimGenControlSchema& Schema, const UEnum* Enum, const FName Tag = TEXT("BitmaskControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyOptionalControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 EncodingSize = 128, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSchemaElement SpecifyEitherControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement A, const FAnimGenControlSchemaElement B, const int32 EncodingSize = 128, const FName Tag = TEXT("EitherControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyEncodingControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FAnimGenControlSchemaElement Element, const int32 EncodingSize = 128, const int32 HiddenLayerNum = 1, const EAnimGenActivationFunction ActivationFunction = EAnimGenActivationFunction::ELU, const FName Tag = TEXT("EncodingControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyBoolControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyFloatControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("FloatControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyLocationControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyRotationControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyScaleControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("ScaleControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyTransformControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("TransformControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyAngleControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("AngleControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyLinearVelocityControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyAngularVelocityControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyDirectionControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyTimeControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("TimeControl"));


	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyLocationsStaticArrayControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Num, const FName Tag = TEXT("LocationsStaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyLinearVelocitiesStaticArrayControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Num, const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyDirectionsStaticArrayControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 Num, const FName Tag = TEXT("DirectionsStaticArrayControl"));


	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyPoseControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 BoneNum, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyTrajectoryControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 TrajectorySampleNum, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSchemaElement SpecifyLocationTrajectoryControl(UPARAM(ref) FAnimGenControlSchema& Schema, const int32 TrajectorySampleNum, const FName Tag = TEXT("LocationTrajectoryControl"));


	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlSchemaElement SpecifyEventControl(UPARAM(ref) FAnimGenControlSchema& Schema, const FName Tag = TEXT("EventControl"));


public:

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlObjectElement MakeNullControl(UPARAM(ref) FAnimGenControlObject& Object, const FName Tag = TEXT("NullControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeContinuousControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<float>& Values, const FName Tag = TEXT("ContinuousControl"));
	static UE_API FAnimGenControlObjectElement MakeContinuousControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const float> Values, const FName Tag = TEXT("ContinuousControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeNamedExclusiveDiscreteControl(UPARAM(ref) FAnimGenControlObject& Object, const FName ElementName, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeNamedInclusiveDiscreteControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));
	static UE_API FAnimGenControlObjectElement MakeNamedInclusiveDiscreteControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeExclusiveDiscreteControl(UPARAM(ref) FAnimGenControlObject& Object, const int32 DiscreteIndex, const FName Tag = TEXT("ExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeInclusiveDiscreteControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<int32>& DiscreteIndices, const FName Tag = TEXT("InclusiveDiscreteControl"));
	static UE_API FAnimGenControlObjectElement MakeInclusiveDiscreteControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const int32> DiscreteIndices, const FName Tag = TEXT("InclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeCountControl(UPARAM(ref) FAnimGenControlObject& Object, const int32 Num, const int32 MaxNum, const FName Tag = TEXT("CountControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeStructControl(UPARAM(ref) FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeStructControlFromArrays(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("StructControl"));
	static UE_API FAnimGenControlObjectElement MakeStructControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeExclusiveUnionControl(UPARAM(ref) FAnimGenControlObject& Object, const FName ElementName, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ExclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FAnimGenControlObjectElement MakeInclusiveUnionControl(UPARAM(ref) FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "ElementNames,Elements"))
	static UE_API FAnimGenControlObjectElement MakeInclusiveUnionControlFromArrays(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("InclusiveUnionControl"));
	static UE_API FAnimGenControlObjectElement MakeInclusiveUnionControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeStaticArrayControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("StaticArrayControl"));
	static UE_API FAnimGenControlObjectElement MakeStaticArrayControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag = TEXT("StaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FAnimGenControlObjectElement MakeSetControl(UPARAM(ref) FAnimGenControlObject& Object, const TSet<FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FAnimGenControlObjectElement MakeSetControlFromArray(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const FName Tag = TEXT("SetControl"));
	static UE_API FAnimGenControlObjectElement MakeSetControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeNamedSparseControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FName>& ElementNames, const TArray<float>& ElementValues, const FName Tag = TEXT("NamedSparseControl"));
	static UE_API FAnimGenControlObjectElement MakeNamedSparseControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, const TArrayView<const float> ElementValues, const FName Tag = TEXT("NamedSparseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeSparseControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<int32>& Indices, const TArray<float>& Values, const FName Tag = TEXT("SparseControl"));
	static UE_API FAnimGenControlObjectElement MakeSparseControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const int32> Indices, const TArrayView<const float> Values, const FName Tag = TEXT("SparseControl"));


	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakePairControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Key, const FAnimGenControlObjectElement Value, const FName Tag = TEXT("PairControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "Elements"))
	static UE_API FAnimGenControlObjectElement MakeArrayControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Elements, const int32 MaxNum, const FName Tag = TEXT("ArrayControl"));
	static UE_API FAnimGenControlObjectElement MakeArrayControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Elements, const int32 MaxNum, const FName Tag = TEXT("ArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Map"))
	static UE_API FAnimGenControlObjectElement MakeMapControl(UPARAM(ref) FAnimGenControlObject& Object, const TMap<FAnimGenControlObjectElement, FAnimGenControlObjectElement>& Map, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "Keys,Values"))
	static UE_API FAnimGenControlObjectElement MakeMapControlFromArrays(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimGenControlObjectElement>& Keys, const TArray<FAnimGenControlObjectElement>& Values, const FName Tag = TEXT("MapControl"));
	static UE_API FAnimGenControlObjectElement MakeMapControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlObjectElement> Keys, const TArrayView<const FAnimGenControlObjectElement> Values, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeEnumControl(UPARAM(ref) FAnimGenControlObject& Object, const UEnum* Enum, const uint8 EnumValue, const FName Tag = TEXT("EnumControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeBitmaskControl(UPARAM(ref) FAnimGenControlObject& Object, const UEnum* Enum, const int32 BitmaskValue, const FName Tag = TEXT("BitmaskControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeOptionalControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const EAnimGenOptionalControl Option, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 1))
	static UE_API FAnimGenControlObjectElement MakeOptionalNullControl(UPARAM(ref) FAnimGenControlObject& Object, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeOptionalValidControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeOptionalControlOnCondition(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const bool bCondition, const FName Tag = TEXT("OptionalControl"));
	static UE_API FAnimGenControlObjectElement MakeOptionalControlOnConditionWithFunction(FAnimGenControlObject& Object, TFunctionRef<FAnimGenControlObjectElement(FAnimGenControlObject&)> ElementFunction, const bool bCondition, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeEitherControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const EAnimGenEitherControl Either, const FName Tag = TEXT("EitherControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, DisplayName = "Make Either A Control"))
	static UE_API FAnimGenControlObjectElement MakeEitherAControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement A, const FName Tag = TEXT("EitherControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2, DisplayName = "Make Either B Control"))
	static UE_API FAnimGenControlObjectElement MakeEitherBControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement B, const FName Tag = TEXT("EitherControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeEncodingControl(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("EncodingControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeBoolControl(UPARAM(ref) FAnimGenControlObject& Object, const bool bValue, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlObjectElement MakeFloatControl(UPARAM(ref) FAnimGenControlObject& Object, const float Value, const FName Tag = TEXT("FloatControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLocationControl(UPARAM(ref) FAnimGenControlObject& Object, const FVector Location, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeRotationControl(UPARAM(ref) FAnimGenControlObject& Object, const FRotator Rotation, const FRotator RelativeRotation = FRotator::ZeroRotator, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeRotationControlFromQuat(UPARAM(ref) FAnimGenControlObject& Object, const FQuat Rotation, const FQuat RelativeRotation, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeScaleControl(UPARAM(ref) FAnimGenControlObject& Object, const FVector Scale, const FVector RelativeScale = FVector(1, 1, 1), const FName Tag = TEXT("ScaleControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeTransformControl(UPARAM(ref) FAnimGenControlObject& Object, const FTransform Transform, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TransformControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeAngleControl(UPARAM(ref) FAnimGenControlObject& Object, const float Angle, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeAngleControlRadians(UPARAM(ref) FAnimGenControlObject& Object, const float Angle, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLinearVelocityControl(UPARAM(ref) FAnimGenControlObject& Object, const FVector LinearVelocity, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeAngularVelocityControl(UPARAM(ref) FAnimGenControlObject& Object, const FVector AngularVelocity, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeDirectionControl(UPARAM(ref) FAnimGenControlObject& Object, const FVector Direction, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeTimeControl(UPARAM(ref) FAnimGenControlObject& Object, const float Time, const float RelativeTime = 0.0f, const FName Tag = TEXT("TimeControl"));


	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLocationsStaticArrayControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& Locations, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationsStaticArrayControl"));
	static UE_API FAnimGenControlObjectElement MakeLocationsStaticArrayControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationsStaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLinearVelocitiesStaticArrayControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& LinearVelocities, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));
	static UE_API FAnimGenControlObjectElement MakeLinearVelocitiesStaticArrayControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> LinearVelocities, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeDirectionsStaticArrayControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& Directions, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionsStaticArrayControl"));
	static UE_API FAnimGenControlObjectElement MakeDirectionsStaticArrayControlFromArrayView(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionsStaticArrayControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakePoseControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& Locations, const TArray<FVector>& LinearVelocities, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("PoseControl"));
	static UE_API FAnimGenControlObjectElement MakePoseControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const TArrayView<const FVector> LinearVelocities, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakePoseControlFromPoseState(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const TArray<int32>& BoneIndices, const FName Tag = TEXT("PoseControl"));
	static UE_API FAnimGenControlObjectElement MakePoseControlFromPoseStateArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const TArrayView<const int32> BoneIndices, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeTrajectoryControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& Locations, const TArray<FVector>& Directions, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TrajectoryControl"));
	static UE_API FAnimGenControlObjectElement MakeTrajectoryControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const TArrayView<const FVector> Directions, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLocationTrajectoryControl(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FVector>& Locations, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationTrajectoryControl"));
	static UE_API FAnimGenControlObjectElement MakeLocationTrajectoryControlFromArrayViews(UPARAM(ref) FAnimGenControlObject& Object, const TArrayView<const FVector> Locations, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 6, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeTrajectoryControlFromTransformTrajectory(UPARAM(ref) FAnimGenControlObject& Object, const FTransformTrajectory& TransformTrajectory, const FTransform& RelativeTransform = FTransform(), const int32 SampleNum = 4, const float FutureTime = 1.0f, const float PastTime = 0.0f, const FVector ForwardVector = FVector(0,1,0), const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5, AutoCreateRefTerm = "RelativeTransform"))
	static UE_API FAnimGenControlObjectElement MakeLocationTrajectoryControlFromTransformTrajectory(UPARAM(ref) FAnimGenControlObject& Object, const FTransformTrajectory& TransformTrajectory, const FTransform& RelativeTransform = FTransform(), const int32 SampleNum = 4, const float FutureTime = 1.0f, const float PastTime = 0.0f, const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeEventControl(UPARAM(ref) FAnimGenControlObject& Object, const bool bTimeUntilEventKnown, const float TimeUntilEvent, const FName Tag = TEXT("EventControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlObjectElement MakeEventControlFromPoseState(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabasePoseState& PoseState, const int32 FrameAttributeIndex, const FName Tag = TEXT("EventControl"));

public:

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 2, ReturnDisplayName = "Success"))
	static UE_API bool GetNullControl(const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NullControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetContinuousControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ContinuousControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetContinuousControl(TArray<float>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ContinuousControl"));
	static UE_API bool GetContinuousControlToArrayView(TArrayView<float> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ContinuousControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedExclusiveDiscreteControl(FName& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedInclusiveDiscreteControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedInclusiveDiscreteControl(TArray<FName>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));
	static UE_API bool GetNamedInclusiveDiscreteControlToArrayView(TArrayView<FName> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetExclusiveDiscreteControl(int32& OutIndex, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveDiscreteControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveDiscreteControl(TArray<int32>& OutIndices, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveDiscreteControl"));
	static UE_API bool GetInclusiveDiscreteControlToArrayView(TArrayView<int32> OutIndices, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetCountControl(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag = TEXT("CountControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructControl(TMap<FName, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructControlToArrays(TArray<FName>& OutElementNames, TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StructControl"));
	static UE_API bool GetStructControlToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructControlElement(FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetExclusiveUnionControl(FName& OutElementName, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ExclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionControl(TMap<FName, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionControlToArrays(TArray<FName>& OutElementNames, TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveUnionControl"));
	static UE_API bool GetInclusiveUnionControlToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success"))
	static UE_API bool FindInclusiveUnionControlElement(FAnimGenControlObjectElement& OutElement, bool& bFound, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ExpandBoolAsExecs = "bFound", ReturnDisplayName = "Success"))
	static UE_API bool SwitchOnHasInclusiveUnionControlElement(FAnimGenControlObjectElement& OutElement, bool& bFound, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName ElementName, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayControl(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StaticArrayControl"));
	static UE_API bool GetStaticArrayControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("StaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetControl(TSet<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetControlToArray(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SetControl"));
	static UE_API bool GetSetControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedSparseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedSparseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedSparseControl(TArray<FName>& OutElements, TArray<float>& OutElementValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedSparseControl"));
	static UE_API bool GetNamedSparseControlToArrayView(TArrayView<FName> OutElements, TArrayView<float> OutElementValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("NamedSparseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSparseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SparseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSparseControl(TArray<int32>& OutIndices, TArray<float>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SparseControl"));
	static UE_API bool GetSparseControlToArrayView(TArrayView<int32> OutIndices, TArrayView<float> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("SparseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetPairControl(FAnimGenControlObjectElement& OutKey, FAnimGenControlObjectElement& OutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("PairControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("ArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetArrayControl(TArray<FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag = TEXT("ArrayControl"));
	static UE_API bool GetArrayControlToArrayView(TArrayView<FAnimGenControlObjectElement> OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const int32 MaxNum, const FName Tag = TEXT("ArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetMapControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetMapControl(TMap<FAnimGenControlObjectElement, FAnimGenControlObjectElement>& OutElements, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetMapControlToArrays(TArray<FAnimGenControlObjectElement>& OutKeys, TArray<FAnimGenControlObjectElement>& OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("MapControl"));
	static UE_API bool GetMapControlToArrayViews(TArrayView<FAnimGenControlObjectElement> OutKeys, TArrayView<FAnimGenControlObjectElement> OutValues, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("MapControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetEnumControl(uint8& OutEnumValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const UEnum* Enum, const FName Tag = TEXT("EnumControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetBitmaskControl(int32& OutBitmaskValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const UEnum* Enum, const FName Tag = TEXT("BitmaskControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ExpandEnumAsExecs = "OutOption", ReturnDisplayName = "Success"))
	static UE_API bool GetOptionalControl(EAnimGenOptionalControl& OutOption, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ExpandEnumAsExecs = "OutEither", ReturnDisplayName = "Success"))
	static UE_API bool GetEitherControl(EAnimGenEitherControl& OutEither, FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("EitherControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetEncodingControl(FAnimGenControlObjectElement& OutElement, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("EncodingControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetBoolControl(bool& bOutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetFloatControl(float& OutValue, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("FloatControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetLocationControl(FVector& OutLocation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetRotationControl(FRotator& OutRotation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FRotator RelativeRotation = FRotator::ZeroRotator, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetRotationControlAsQuat(FQuat& OutRotation, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FQuat RelativeRotation, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetScaleControl(FVector& OutScale, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FVector RelativeScale = FVector(1, 1, 1), const FName Tag = TEXT("ScaleControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetTransformControl(FTransform& OutTransform, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TransformControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetAngleControl(float& OutAngle, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetAngleControlRadians(float& OutAngle, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetLinearVelocityControl(FVector& OutLinearVelocity, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetAngularVelocityControl(FVector& OutAngularVelocity, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetDirectionControl(FVector& OutDirection, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetTimeControl(float& OutTime, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const float RelativeTime = 0.0f, const FName Tag = TEXT("TimeControl"));



	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetLocationsStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("LocationsStaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetLocationsStaticArrayControl(TArray<FVector>& OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationsStaticArrayControl"));
	static UE_API bool GetLocationsStaticArrayControlToArrayView(TArrayView<FVector> OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationsStaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetLinearVelocitiesStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetLinearVelocitiesStaticArrayControl(TArray<FVector>& OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));
	static UE_API bool GetLinearVelocitiesStaticArrayControlToArrayView(TArrayView<FVector> OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LinearVelocitiesStaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetDirectionsStaticArrayControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("DirectionsStaticArrayControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetDirectionsStaticArrayControl(TArray<FVector>& OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionsStaticArrayControl"));
	static UE_API bool GetDirectionsStaticArrayControlToArrayView(TArrayView<FVector> OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionsStaticArrayControl"));


	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetPoseControlNum(int32& OutNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetPoseControl(TArray<FVector>& OutLocations, TArray<FVector>& OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("PoseControl"));
	static UE_API bool GetPoseControlToArrayViews(TArrayView<FVector> OutLocations, TArrayView<FVector> OutLinearVelocities, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("PoseControl"));


	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetTrajectoryControlNum(int32& OutLocationNum, int32& OutDirectionNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetTrajectoryControl(TArray<FVector>& OutLocations, TArray<FVector>& OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TrajectoryControl"));
	static UE_API bool GetTrajectoryControlToArrayViews(TArrayView<FVector> OutLocations, TArrayView<FVector> OutDirections, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("TrajectoryControl"));


	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetLocationTrajectoryControlNum(int32& OutLocationNum, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool GetLocationTrajectoryControl(TArray<FVector>& OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationTrajectoryControl"));
	static UE_API bool GetLocationTrajectoryControlToArrayViews(TArrayView<FVector> OutLocations, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FName Tag = TEXT("LocationTrajectoryControl"));


	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetEventControl(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Tag = TEXT("EventControl"));

public:

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeLocationDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& LocationFrameAttribute, const float Scale = 100.0f, const float Weight = 1.0f, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeVelocityDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& VelocityFrameAttribute, const float Scale = 200.0f, const float Weight = 1.0f, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeDirectionDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute, const float Weight = 1.0f, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeRotationDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& RotationFrameAttribute, const float Weight = 1.0f, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeEventDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabaseFrameAttribute& FrameAttribute, const float Weight = 1.0f, const FName Tag = TEXT("EventControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 8, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributeTrajectoryDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& DirectionFrameAttributes, const float LocationScale = 100.0f, const float LocationWeight = 1.0f, const float DirectionWeight = 1.0f, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 10, ReturnDisplayName = "Success"))
	static UE_API bool FrameAttributePoseDistanceFromControl(FAnimDatabaseFrameAttribute& OutFrameAttribute, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& LinearVelocityFrameAttributes, const float LocationScale = 100.0f, const float LinearVelocityScale = 100.0f, const float LocationWeight = 1.0f, const float LinearVelocityWeight = 1.0f, const float BlendTime = 0.2f, const FName Tag = TEXT("PoseControl"));

public:

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimDatabaseFrameRanges ControlSetFrameRanges(const FAnimGenControlSet& ControlSet);

	UFUNCTION(BlueprintPure, Category = "AnimGen")
	static UE_API FAnimGenControlSet MakeEmptyControlSet();

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeNullControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetFrameRanges, const FName Tag = TEXT("NullControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeBoolControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& BoolFrameAttribute, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeFloatControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const FName Tag = TEXT("FloatControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeLocationControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& LocationFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeRotationControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& RotationFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeLinearVelocityControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& LinearVelocityFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeAngularVelocityControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& AngularVelocityFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeDirectionControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeTimeControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& TimeFrameAttribute, const FAnimDatabaseFrameAttribute& RelativeTimeFrameAttribute, const FName Tag = TEXT("TimeControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeExclusiveDiscreteControlSetFromConstant(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetFrameRanges, const int32 DiscreteIndex, const int32 Size, const FName Tag = TEXT("ExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeTrajectoryControlSet(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& DirectionFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("TrajectoryControl"));
	static UE_API FAnimGenControlSet MakeTrajectoryControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> DirectionFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeLocationTrajectoryControlSet(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("LocationTrajectoryControl"));
	static UE_API FAnimGenControlSet MakeLocationTrajectoryControlSetFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakePoseControlSet(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimDatabaseFrameAttribute>& LocationFrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& LinearVelocityFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("PoseControl"));
	static UE_API FAnimGenControlSet MakePoseControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FAnimDatabaseFrameAttribute> LocationFrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> LinearVelocityFrameAttributes, const FAnimDatabaseFrameAttribute& RelativeTransformFrameAttribute, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeStructControlSet(UPARAM(ref) FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlSet>& Elements, const FName Tag = TEXT("StructControl"));
	static UE_API FAnimGenControlSet MakeStructControlSetFromArrays(FAnimGenControlObject& Object, const TArray<FName>& ElementNames, TArray<FAnimGenControlSet>& Elements, const FName Tag = TEXT("StructControl"));
	static UE_API FAnimGenControlSet MakeStructControlSetFromArrayViews(FAnimGenControlObject& Object, const TArrayView<const FName> ElementNames, TArrayView<const FAnimGenControlSet> Elements, const FName Tag = TEXT("StructControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeExclusiveUnionControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FName ElementName, const FAnimGenControlSet& Element, const FName Tag = TEXT("ExclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeEmptyInclusiveUnionControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetFrameRanges, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeInclusiveUnionControlSet(UPARAM(ref) FAnimGenControlObject& Object, const TMap<FName, FAnimGenControlSet>& Elements, const FName Tag = TEXT("InclusiveUnionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeOptionalControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlSet& InControlSet, const FAnimDatabaseFrameAttribute& ValidFrameAttribute, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeOptionalNullControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& ControlSetFrameRanges, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeOptionalValidControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlSet& InControlSet, const FName Tag = TEXT("OptionalControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeEnumControlSet(UPARAM(ref) FAnimGenControlObject& Object, const UEnum* Enum, const uint8 EnumValue, const FAnimDatabaseFrameRanges& ControlSetFrameRanges, const FName Tag = TEXT("EnumControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeEventControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameAttribute& EventFrameAttribute, const FName Tag = TEXT("EventControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeSetControlSetFromArray(UPARAM(ref) FAnimGenControlObject& Object, const TArray<FAnimGenControlSet>& Elements, const TArray<FAnimDatabaseFrameAttribute>& ElementMasks, const FName Tag = TEXT("SetControl"));
	static UE_API FAnimGenControlSet MakeSetControlSetFromArrayView(FAnimGenControlObject& Object, const TArrayView<const FAnimGenControlSet> Elements, const TArrayView<const FAnimDatabaseFrameAttribute> ElementMasks, const FName Tag = TEXT("SetControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 2))
	static UE_API FAnimGenControlSet MakeEncodingControlSet(UPARAM(ref) FAnimGenControlObject& Object, const FAnimGenControlSet& Element, const FName Tag = TEXT("EncodingControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeNamedInclusiveDiscreteControlSetFromFrameRanges(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameRanges>& ElementMap, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));
	static UE_API FAnimGenControlSet MakeNamedInclusiveDiscreteControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameRanges> ElementFrameRanges, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeNamedInclusiveDiscreteControlSetFromAnimNotifyStates(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, TSubclassOf<UAnimNotifyState>>& AnimNotifyStateMap, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));
	static UE_API FAnimGenControlSet MakeNamedInclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> ElementAnimNotifyStates, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeNamedExclusiveDiscreteControlSetFromFrameRanges(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameRanges>& ElementMap, const FName DefaultElementName, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));
	static UE_API FAnimGenControlSet MakeNamedExclusiveDiscreteControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameRanges> ElementFrameRanges, const FName DefaultElementName, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));
	
	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeNamedExclusiveDiscreteControlSetFromAnimNotifyStates(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, TSubclassOf<UAnimNotifyState>>& AnimNotifyStateMap, const FName DefaultElementName, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));
	static UE_API FAnimGenControlSet MakeNamedExclusiveDiscreteControlSetFromAnimNotifyStatesArrayViews(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> ElementAnimNotifyStates, const FName DefaultElementName, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeNamedSparseControlSetFromFrameRanges(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TMap<FName, FAnimDatabaseFrameAttribute>& ElementMap, const FName Tag = TEXT("NamedSparseControl"));
	static UE_API FAnimGenControlSet MakeNamedSparseControlSetFromFrameRangesArrayViews(FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> ElementNames, const TArrayView<const FAnimDatabaseFrameAttribute> ElementFrameValues, const FName Tag = TEXT("NamedSparseControl"));

public:

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeRootLinearVelocityControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeRootAngularVelocityControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeFutureRootLinearVelocityControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float FutureTime = 0.25f, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeFutureRootDirectionControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float FutureTime = 0.25f, const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeRootLocationAtRangeEndControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeRootDirectionAtRangeEndControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FVector ForwardVector = FVector(0.0, 1.0f, 0.0f), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 7))
	static UE_API FAnimGenControlSet MakeTrajectoryControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 SampleNum = 4, const float FutureTime = 1.0f, const float PastTime = 0.0f, const FVector ForwardVector = FVector(0.0, 1.0f, 0.0f), const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 9))
	static UE_API FAnimGenControlSet MakeLookAtTrajectoryControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 LookAtBoneIndex, const int32 SampleNum = 4, const float FutureTime = 1.0f, const float PastTime = 0.0f, const FVector LocalDirection = FVector(0.0, 1.0f, 0.0f), const float DirectionSmoothingAmount = 15.0f, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeLocationTrajectoryControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 SampleNum = 4, const float FutureTime = 1.0f, const float PastTime = 0.0f, const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeBoolControlSetFromConstant(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& FrameRanges, const bool bValue, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeBoolControlSetFromMirrored(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeBoolControlSetFromNotMirrored(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 3))
	static UE_API FAnimGenControlSet MakeBoolControlSetFromActiveRanges(UPARAM(ref) FAnimGenControlObject& Object, const FAnimDatabaseFrameRanges& Active, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeBoolControlSetFromAnimNotifyState(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const TSubclassOf<UAnimNotifyState> AnimNotifyState, const FAnimDatabaseFrameRanges& FrameRanges, const FName Tag = TEXT("BoolControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeBoneGlobalLocationControlSet(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeBoneGlobalLocationControlSetFromName(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeBoneGlobalLocationAtNearestFramesControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FAnimDatabaseFrames& Frames, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeBoneGlobalLocationAtNearestFramesControlSetFromDatabaseBoneName(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FAnimDatabaseFrames& Frames, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 6))
	static UE_API FAnimGenControlSet MakeBoneGlobalDirectionAtNearestFramesControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FAnimDatabaseFrames& Frames, const FVector LocalDirection = FVector::ForwardVector, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 6))
	static UE_API FAnimGenControlSet MakeBoneGlobalDirectionAtNearestFramesControlSetFromDatabaseBoneName(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FAnimDatabaseFrames& Frames, const FVector LocalDirection = FVector::ForwardVector, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeBoneGlobalDirectionControlSet(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FVector ForwardVector = FVector(0.0, 1.0f, 0.0f), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 5))
	static UE_API FAnimGenControlSet MakeBoneGlobalDirectionControlSetFromName(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FVector ForwardVector = FVector(0.0, 1.0f, 0.0f), const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakePoseControlSetFromDatabase(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const FName Tag = TEXT("PoseControl"));
	static UE_API FAnimGenControlSet MakePoseControlSetFromDatabaseArrayView(FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure, Category = "AnimGen", meta = (AdvancedDisplay = 4))
	static UE_API FAnimGenControlSet MakeTimeUntilNearestAnimNotifyControlSet(UPARAM(ref) FAnimGenControlObject& Object, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, TSubclassOf<UAnimNotify> AnimNotifyClass, const FName Tag = TEXT("TimeControl"));


public:

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugLocationControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FLinearColor DrawColor = FLinearColor::Red, const float DrawRadius = 10.0f, const float Thickness = 0.0f, const int32 SegmentNum = 8, const FName Tag = TEXT("LocationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugRotationControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FVector DrawLocation = FVector::ZeroVector, const float DrawRadius = 10.0f, const float Thickness = 0.0f, const FName Tag = TEXT("RotationControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugLinearVelocityControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FVector DrawLocation = FVector::ZeroVector, const FLinearColor DrawColor = FLinearColor::Red, const float DrawVelocityScale = 1.0f, const float Thickness = 0.0f, const FName Tag = TEXT("LinearVelocityControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugAngularVelocityControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FVector DrawLocation = FVector::ZeroVector, const FLinearColor DrawColor = FLinearColor::Red, const float DrawVelocityScale = 1.0f, const float Thickness = 0.0f, const FName Tag = TEXT("AngularVelocityControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugDirectionControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FVector DrawLocation = FVector::ZeroVector, const FLinearColor DrawColor = FLinearColor::Red, const float DrawArrowLength = 100.0f, const float ArrowHeadScale = 1.0f, const float Thickness = 0.0f, const FName Tag = TEXT("DirectionControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success"))
	static UE_API bool DrawDebugPoseControlFromPoseState(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabasePoseState& PoseState, const TArray<int32>& BoneIndices, const FLinearColor DrawColor = FLinearColor::Red, const float DrawVelocityLineScale = 0.1f, const float Thickness = 0.0f, const FName Tag = TEXT("PoseControl"));
	static UE_API bool DrawDebugPoseControlFromPoseStateArrayViews(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FAnimDatabasePoseState& PoseState, const TArrayView<const int32> BoneIndices, const FLinearColor DrawColor = FLinearColor::Red, const float DrawVelocityLineScale = 0.1f, const float Thickness = 0.0f, const FName Tag = TEXT("PoseControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugTrajectoryControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FLinearColor DrawColor = FLinearColor::Red, const float DrawArrowLength = 25.0f, const float PointRadius = 5.0f, const float ArrowHeadScale = 1.0f, const float Thickness = 0.0f, const int32 SegmentNum = 8, const FName Tag = TEXT("TrajectoryControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success", AutoCreateRefTerm = "RelativeTransform"))
	static UE_API bool DrawDebugLocationTrajectoryControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FTransform& RelativeTransform = FTransform(), const FLinearColor DrawColor = FLinearColor::Red, const float PointRadius = 5.0f, const float Thickness = 0.0f, const int32 SegmentNum = 8, const FName Tag = TEXT("LocationTrajectoryControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 8, ReturnDisplayName = "Success", AutoCreateRefTerm = "Location, Rotation"))
	static UE_API bool DrawDebugNamedExclusiveDiscreteControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale = 10.0f, const FLinearColor DrawColor = FLinearColor::Red, const float Thickness = 0.0f, const FName Tag = TEXT("NamedExclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 8, ReturnDisplayName = "Success", AutoCreateRefTerm = "Location, Rotation"))
	static UE_API bool DrawDebugNamedInclusiveDiscreteControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale = 10.0f, const FLinearColor DrawColor = FLinearColor::Red, const float Thickness = 0.0f, const FName Tag = TEXT("NamedInclusiveDiscreteControl"));

	UFUNCTION(BlueprintPure = false, Category = "AnimGen", meta = (AdvancedDisplay = 8, ReturnDisplayName = "Success", AutoCreateRefTerm = "Location, Rotation"))
	static UE_API bool DrawDebugNamedSparseControl(const FDebugDrawer& Drawer, const FAnimGenControlObject& Object, const FAnimGenControlObjectElement Element, const FName Label, const FVector& Location, const FRotator& Rotation, const float Scale = 10.0f, const FLinearColor DrawColor = FLinearColor::Red, const float Thickness = 0.0f, const FName Tag = TEXT("NamedSparseControl"));
};

#undef UE_API