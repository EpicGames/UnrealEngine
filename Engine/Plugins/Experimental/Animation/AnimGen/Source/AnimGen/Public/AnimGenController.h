// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGenTraining.h"
#include "AnimGenControl.h"

#include "AnimDatabase.h"
#include "AnimDatabasePose.h"

#include "LearningArray.h"

#include "Engine/DataAsset.h"

#include "AnimGenController.generated.h"

#define UE_API ANIMGEN_API

class ULearningNeuralNetworkData;
class UAnimGenAutoEncoder;
class UAnimGenBehavior;
class UAnimGenBehaviorPreview;

namespace UE::AnimDatabase::Editor
{
	struct FQueryEntry;
}

/** Class containing all the UAnimGenController Editor window viewport settings */
UCLASS()
class UAnimGenControllerViewportSettings : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Mesh to use for previewing the character */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TObjectPtr<USkeletalMesh> PreviewMesh = nullptr;

public:

	/** If to draw the skeleton of all characters */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	bool bDrawSkeleton = false;

	/** If to only draw the bones that are being encoded by the auto-encoder */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawRequiredBonesOnly = true;

	/** Draw a simple skeleton made up of lines connecting bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawSimpleSkeleton = false;

	/** Opacity of the drawn skeleton */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	float DrawSkeletonOpacity = 1.0f;

	/** The scale of the drawn skeleton bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	float DrawSkeletonScale = 1.0f;

	/** If to draw linear velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawLinearVelocities = false;

	/** The scale of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesScale = 0.05f;

	/** Opacity of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesOpacity = 0.25f;

	/** If to draw angular velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawAngularVelocities = false;

	/** The scale of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesScale = 2.5f;

	/** Opacity of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesOpacity = 0.25f;

	/** If to draw bone transforms */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawBoneTransforms = false;

public:

	/** If to draw the root of the skeleton as a transform */
	UPROPERTY(EditAnywhere, Category = "Root")
	bool bDrawRoot = false;

	/** Opacity of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootOpacity = 1.0f;

	/** The scale of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootScale = 10.0f;

public:

	/** If to draw the trajectories of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	bool bDrawTrajectories = false;

	/** If to draw the trajectory orientations of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (EditCondition = "bDrawTrajectories", HideEditConditionToggle))
	bool bDrawTrajectoryOrientations = false;

public:

	/** Re-orients all ranges so that they start at the scene origin rather than wherever they started in the source data */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bRangesStartAtOrigin = true;

	/** Offset different ranges away from origin so they can be viewed together */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bOffsetRanges = true;

	/** Spacing to use for offsetting ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bOffsetRanges", HideEditConditionToggle))
	float RangeOffsetSpacing = 100.0f;

	/** If to offset selected ranges as a grid */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOffsetRanges", HideEditConditionToggle))
	bool bOffsetRangesInGrid = false;

	/** Remove root translation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bDisableRootTranslation = false;

	/** Remove root rotation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin && bDisableRootTranslation", HideEditConditionToggle))
	bool bDisableRootRotation = false;

	/** If to draw the colored range identifier over the character to show which range in the query the animation is from */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bDrawRangeIdentifier = true;

	/** Seed used for the colors of the range identifiers */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	int32 RangeIdentifierColorSeed = 1234;

	/** Thickness of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierThickness = 1.0f;

	/** Radius of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierRadius = 2.0f;

	/** Opacity of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierOpacity = 0.5f;

	/** Height offset used to draw the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierHeight = 200.0f;

	/** If to override the range identifier color */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bOverrideRangeIdentifierColor = false;

	/** Color to use to override the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOverrideRangeIdentifierColor", HideEditConditionToggle))
	FLinearColor RangeIdentifierColorOverride = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);

public:

	/** If to draw the scene origin as a transform */
	UPROPERTY(EditAnywhere, Category = "Origin")
	bool bDrawOrigin = true;

	/** Thickness of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginLineThickness = 1.0f;

	/** Opacity of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginOpacity = 0.5f;

	/** Scale of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginScale = 25.0f;

public:

	/** If to call the draw debug function on the behavior */
	UPROPERTY(EditAnywhere, Category = "Behavior")
	bool bDrawDebugTesting = true;

	/** If to call the draw debug function on the behavior preview */
	UPROPERTY(EditAnywhere, Category = "Behavior")
	bool bDrawDebugTraining = true;

public:

	/** If to draw a representation of the encoded control vector */
	UPROPERTY(EditAnywhere, Category = "Control Encoding")
	bool bDrawEncodedControlVector = false;

	/** How many dimensions of the encoded control vector to draw */
	UPROPERTY(EditAnywhere, Category = "Control Encoding", meta = (ClampMin = "1", UIMin = "1", ClampMax = "16", UIMax = "16", EditCondition = "bDrawEncodedControlVector", HideEditConditionToggle))
	int32 DrawEncodedControlVectorDimensions = 8;

	/** Scale of the drawing for the encoded control vector */
	UPROPERTY(EditAnywhere, Category = "Control Encoding", meta = (ClampMin = "0", UIMin = "10", ClampMax = "100", UIMax = "100", EditCondition = "bDrawEncodedControlVector", HideEditConditionToggle))
	float DrawEncodedControlVectorScale = 10.0f;

	/** Color of the drawing for the encoded control vector */
	UPROPERTY(EditAnywhere, Category = "Control Encoding", meta = (EditCondition = "bDrawEncodedControlVector", HideEditConditionToggle))
	FLinearColor DrawEncodedControlVectorColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);

#endif
};

/** Class containing all the UAnimGenController Editor window training settings */
UCLASS()
class UAnimGenControllerTrainingSettings : public UObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	/** Forces the Update function to refresh the internal state */
	UFUNCTION(CallInEditor, Category = "Data", DisplayName = "Refresh")
	UE_API void ForceRefresh();

	/** Load the current database. */
	UE_API void LoadDatabaseAsync();

	/** Load the current auto-encoder. */
	UE_API void LoadAutoEncoderAsync();

	/** We use this async load changes to Database/AutoEncoder */
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& Event) override;

#endif

#if WITH_EDITORONLY_DATA

	/** Input database to train the network with */
	UPROPERTY(EditAnywhere, Category = "Data")
	TSoftObjectPtr<UAnimDatabase> Database = nullptr;

	/** Frame ranges in the database to use for training. Leave this as None to train on the full database */
	UPROPERTY(EditAnywhere, Instanced, Category = "Data")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges = nullptr;

	/** Input autoencoder to train the network with */
	UPROPERTY(EditAnywhere, Category = "Data")
	TSoftObjectPtr<UAnimGenAutoEncoder> AutoEncoder = nullptr;

	/** Input Behavior to train the network on */
	UPROPERTY(EditAnywhere, Instanced, Category = "Data")
	TObjectPtr<UAnimGenBehavior> Behavior = nullptr;

public:

	/** Flag to force refresh the query during update */
	bool bForceRefresh = false;

	/** This hash is used to detect when the database might have changed and re-generate the control sets */
	int32 DatabaseContentHash = 0;

	/** Query Ranges object for the current query */
	FAnimDatabaseFrameRanges QueryRanges;

	/** Current Control Schema */
	FAnimGenControlSchema ControlSchema;

	/** Current Control Schema Element */
	FAnimGenControlSchemaElement ControlSchemaElement;

	/** Current Control Object */
	FAnimGenControlObject ControlObject;

	/** Current Control Sets */
	TArray<FAnimGenControlSet> ControlSets;

	/** List of UI entries for the current query */
	TArray<TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>> QueryEntries;

	/** Indices of all the currently selected ranges */
	TArray<int32> SelectedRanges;

	/** Seed used for the colors of the range identifiers */
	int32 RangeIndentifierColorSeed = 1234;

public:

	/** Custom debug draw object that can be used to add additional debug draw behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Debug Draw")
	TObjectPtr<UAnimDatabaseDebugDraw> DebugDrawer;

#endif

#if WITH_EDITOR

	/**
	 * Checks if ControlSets needs updating, and if so updates it. returns true if ControlSets was updated (and therefore the UI may need updating),
	 * otherwise false.
	 */
	UE_API bool Update();

private:

	/** Internal function for updating the QueryEntries array from an updated QueryRanges */
	UE_API void UpdateQueryEntries();

#endif

public:

#if WITH_EDITORONLY_DATA

	/** The number of hidden units used by the denoiser network on each layer */
	UPROPERTY(EditAnywhere, Category = "Network", meta = (ClampMin = "1", UIMin = "1"))
	int32 DenoiserHiddenUnitNum = 1792;

	/** The number of layers used by the denoiser network */
	UPROPERTY(EditAnywhere, Category = "Network", meta = (ClampMin = "2", UIMin = "2"))
	int32 DenoiserLayerNum = 10;

	/** The number of hidden units used by the LOD0 network on each layer */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD0 Hidden Unit Num", meta = (ClampMin = "1", UIMin = "1"))
	int32 LOD0HiddenUnitNum = 1024;

	/** The number of layers used by the LOD0 network */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD0 Layer Num", meta = (ClampMin = "2", UIMin = "2"))
	int32 LOD0LayerNum = 8;

	/** The number of hidden units used by the LOD1 network on each layer */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD1 Hidden Unit Num", meta = (ClampMin = "1", UIMin = "1"))
	int32 LOD1HiddenUnitNum = 512;

	/** The number of layers used by the LOD1 network */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD1 Layer Num", meta = (ClampMin = "2", UIMin = "2"))
	int32 LOD1LayerNum = 6;

	/** The number of hidden units used by the LOD2 network on each layer */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD2 Hidden Unit Num", meta = (ClampMin = "1", UIMin = "1"))
	int32 LOD2HiddenUnitNum = 256;

	/** The number of layers used by the LOD2 network */
	UPROPERTY(EditAnywhere, Category = "Network", DisplayName = "LOD2 Layer Num", meta = (ClampMin = "2", UIMin = "2"))
	int32 LOD2LayerNum = 4;

public:

	/**
	 * If the controller should use compressed learning layers.
	 * This reduces the memory use by half at some very small cost to accuracy.
	 */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	bool bUseCompressedLinearLayers = true;

	/** The activation function used by the networks */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	EAnimGenActivationFunction ActivationFunction = EAnimGenActivationFunction::GELU;

	/** Network weight initialization method */
	UPROPERTY(EditAnywhere, Category = "Network", AdvancedDisplay)
	EAnimGenWeightInit WeightInit = EAnimGenWeightInit::Uniform;

public:

	/**
	 * The number of iterations to train the reference flow-matching network. Typical values are between 100000 and 2000000. Larger values
	 * will improve motion fideltity and reduce noise, but can result in over-fitting where the character stuggles to transition between motions.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumberOfTrainingIterations = 500000;

	/**
	 * The number of iterations to train the distilled LOD networks. Typical values are between 100000 and 1000000. Larger values will improve
	 * motion quality but can increase training time drastically.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumberOfDistillationIterations = 200000;

	/** Learning rate. Typical values are between 0.001 and 0.0001. Reduce if training is unsable. Generally this does not need adjusting. */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRate = 0.001f;

	/**
	 * Batch size to use for training. Smaller values can sometimes produce better results at the cost of slowing down training. Large batch 
	 * sizes are much more computationally efficient when training on the GPU. Reduce this if you are getting GPU out-of-memory errors during 
	 * training. Generally this should not need adjusting.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "1", UIMin = "1", UIMax = "16384"))
	int32 BatchSize = 512;

	/**
	 * The scale of noise to apply to the pose encoding during training. Larger values may allow the character to transition better between 
	 * motions at the expense of making the result more noisy with less motion fidelity.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float PoseNoiseScale = 0.1f;

	/**
	 * Rate at which to sample random previous poses during training. Setting this to a value larger than zero can greatly help the character 
	 * transition between motions when the control input changes. However this can also cause the character to randomly transition at undesiable
	 * times too so you may want to set this to zero this if you prefer more strict pose continuity such for an idle controller.
	 */
	UPROPERTY(EditAnywhere, Category = "Training", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RandomPoseSampleRate = 0.1f;

	/**
	* If to enable logging to the UE log in the training subprocess. Enable this if training is failing for an unknown reason as it will allow you 
	* to check the log for errors.
	*/
	UPROPERTY(EditAnywhere, Category = "Training")
	bool bEnableLogging = false;

	/** If to enable logging to TensorBoard in the intermediate directory. This can slow down training but gives important analytics. */
	UPROPERTY(EditAnywhere, Category = "Training")
	bool bEnableTensorboard = false;

public:

	/** The seed used for any random sampling the trainer will perform, e.g. for weight initialization. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "0", UIMin = "0"))
	int32 RandomSeed = 1234;

	/** Scale of noise to apply to the control inputs. Larger values may improve generalization a little. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ControlNoiseScale = 0.01f;

	/** The number of learning rate warm-up iterations to apply. Stabilizes the beginning of training. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 WarmupIterations = 1000;

	/** Number of denoising steps to use during training of distilled LODs. Larger values may improve quality marginally but makes training slower. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 DenoiserSteps = 4;

	/** The device to train on. Training on CPU, while possible, will typically be extremely slow. */
	UPROPERTY(EditAnywhere, Category = "Training", AdvancedDisplay)
	EAnimGenTrainingDevice Device = EAnimGenTrainingDevice::GPU;

#endif
};

/**
 * The Controller class represents a trained character controller that uses flow-matching and a neural-network to auto-regressively produce the 
 * character's pose frame by frame.
 * 
 * A controller is always implicitly linked to a behavior which it was trained on - and input to the controller will be provided via the behavior.
 */
UCLASS(Blueprintable, BlueprintType)
class UAnimGenController : public UDataAsset
{
	GENERATED_BODY()

private:

	UE_API UAnimGenController(const FObjectInitializer& ObjectInitializer);

public:

	/** Check if the controller is valid and trained */
	UFUNCTION(BlueprintPure, Category = "AnimGen")
	UE_API bool IsValid() const;

	/** Reset the controller back to an untrained state */
	UFUNCTION(BlueprintCallable, Category = "AnimGen")
	UE_API void Invalidate();

public:

	/** Normalize encoded pose vectors to make them appropriate for the controller */
	UE_API void NormalizeEncodedPoseVectors(
		const TLearningArrayView<2, float> OutNormalizedEncodedPoseVectors,
		const TLearningArrayView<2, const float> InUnnormalizedEncodedPoseVectors) const;

	/** Normalize encoded pose vectors in-place to make them appropriate for the controller */
	UE_API void NormalizeEncodedPoseVectorsInplace(const TLearningArrayView<2, float> InOutEncodedPoseVectors) const;

	/** De-normalize encoded pose vectors produced by the controller */
	UE_API void DenormalizeEncodedPoseVectors(
		const TLearningArrayView<2, float> OutUnnormalizedEncodedPoseVectors,
		const TLearningArrayView<2, const float> InNormalizedEncodedPoseVectors) const;

	/** Clamp encoded pose vectors produced by the controller to within a valid range */
	UE_API void ClampNormalizedPoseVectorsInplace(const TLearningArrayView<2, float> InOutNormalizedEncodedPoseVectors) const;

	/** Scale initial sampling noise in place to match normalized encoded pose vectors standard deviation */
	UE_API void ScaleSamplingNoiseInplace(const TLearningArrayView<2, float> InOutSamplingNoise) const;

public:

	/** Auto-Encoder which this controller uses */
	UPROPERTY(VisibleAnywhere, Category = "Controller")
	TObjectPtr<UAnimGenAutoEncoder> AutoEncoder = nullptr;

	/** Control Schema for the given Behavior */
	UPROPERTY()
	FAnimGenControlSchema ControlSchema;

	/** Control Schema Element for the given Behavior */
	UPROPERTY()
	FAnimGenControlSchemaElement ControlSchemaElement;

	/** Control Encoder Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> ControlEncoderNetwork = nullptr;

	/** LOD0 Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> LOD0Network = nullptr;

	/** LOD1 Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> LOD1Network = nullptr;

	/** LOD2 Network */
	UPROPERTY(Instanced)
	TObjectPtr<ULearningNeuralNetworkData> LOD2Network = nullptr;

	/** The Frame Rate this controller was trained to run at */
	UPROPERTY(VisibleAnywhere, Category = "Controller")
	FFrameRate FrameRate = FFrameRate(60, 1);

	/** Size of the control vector used by this controller */
	UPROPERTY()
	int32 ControlVectorSize = 0;

	/** Size of the encoded control vector used by this controller */
	UPROPERTY()
	int32 EncodedControlVectorSize = 0;

	/** Size of the control distribution vector used to clamp controls for this controller */
	UPROPERTY()
	int32 ControlDistributionVectorSize = 0;

	/** Mean of the encoded pose vectors used for training */
	UPROPERTY()
	TArray<float> EncodedPoseMeans;

	/** Std of the encoded pose vectors used for training */
	UPROPERTY()
	TArray<float> EncodedPoseStds;

	/** Max of the encoded pose vectors used for training */
	UPROPERTY()
	TArray<float> EncodedPoseMaxs;

	/** Min of the encoded pose vectors used for training */
	UPROPERTY()
	TArray<float> EncodedPoseMins;

	/** Std used for normalizing the encoded pose vectors */
	UPROPERTY()
	TArray<float> NormalizedPoseStds;

	/** Scale used for normalizing encoded pose vectors */
	UPROPERTY()
	float EncodedPoseNormalizationScale = 1.0f;

	/** Min of the control vectors used by this controller */
	UPROPERTY()
	TArray<float> ControlVectorDistributionMins;

	/** Max of the control vectors used by this controller */
	UPROPERTY()
	TArray<float> ControlVectorDistributionMaxs;

public:

#if WITH_EDITORONLY_DATA

	/** The content hash of the frame ranges used for training */
	UPROPERTY()
	int32 TrainedFrameRangesContentHash = 0;

	/** The content hash of the auto-encoder used for training */
	UPROPERTY()
	int32 TrainedAutoEncoderContentHash = 0;

	/** The compatibility hash of the schema used for training */
	UPROPERTY()
	int32 TrainedSchemaCompatibilityHash = 0;

public:

	/** Current size of the trained control encoder in kilobytes */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (ForceUnits = "Kilobytes"))
	int32 ControlEncoderSize = 0;

	/** Current size of all the trained denoiser networks in kilobytes */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (ForceUnits = "Kilobytes"))
	int32 DenoiserSize = 0;

public:

	/** Editor-only object containing the training settings for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimGenControllerTrainingSettings> TrainingSettings;

	/** Editor-only object containing the viewport settings for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimGenControllerViewportSettings> ViewportSettings;

#endif
};

#undef UE_API