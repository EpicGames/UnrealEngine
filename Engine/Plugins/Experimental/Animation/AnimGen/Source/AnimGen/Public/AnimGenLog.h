// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"

#define UE_API ANIMGEN_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimGen, Log, All);

#undef UE_API