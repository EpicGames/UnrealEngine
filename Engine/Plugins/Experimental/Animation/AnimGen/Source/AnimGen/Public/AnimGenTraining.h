// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGenTraining.generated.h"

/** Enumeration of the training devices. */
UENUM(BlueprintType, Category = "AnimGen")
enum class EAnimGenTrainingDevice : uint8
{
	CPU,
	GPU,
};

/** Enumeration of activation functions. */
UENUM(BlueprintType, Category = "AnimGen")
enum class EAnimGenActivationFunction : uint8
{
	/** ELU Activation - Generally performs better than ReLU and is not prone to gradient collapse but slower to evaluate. */
	ELU		UMETA(DisplayName = "ELU"),
	
	/** ReLU Activation - Fast to train and evaluate but occasionally causes gradient collapse and untrainable networks. */
	ReLU	UMETA(DisplayName = "ReLU"),

	/** TanH Activation - Smooth activation function that is slower to train and evaluate but sometimes more stable for certain tasks. */
	TanH	UMETA(DisplayName = "TanH"),

	/** GELU Activation - Similar to ELU but sometimes can perform better for deeper networks. */
	GELU	UMETA(DisplayName = "GELU"),
};

/** Enumeration of different weight initialization. */
UENUM(BlueprintType, Category = "AnimGen")
enum class EAnimGenWeightInit : uint8
{
	Gaussian = 0,
	Uniform = 1
};
