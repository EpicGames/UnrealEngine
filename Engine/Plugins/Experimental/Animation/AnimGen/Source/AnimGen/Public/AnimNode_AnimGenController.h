// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Misc/FrameRate.h"

#include "AnimDatabasePose.h"

#include "AnimGenController.h"

#include "AnimNode_AnimGenController.generated.h"

#define UE_API ANIMGEN_API

namespace UE::Anim { class IAnimRootMotionProvider; }
namespace UE::Learning
{
	struct FNeuralNetworkInference;
}

/** Specifies a curve output from the controller */
USTRUCT(BlueprintType)
struct FAnimGenControllerCurveOutput
{
	GENERATED_BODY()

public:

	/** Auto-Encoder Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName AutoEncoderAttributeName = NAME_None;

	/** Output if the provided attribute is active or not */
	UPROPERTY(EditAnywhere, Category = "Entry")
	bool bOutputAutoEncoderAttributeActive = false;

	/** Output Curve name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName OutputCurveName = NAME_None;
};

/** Specifies an attribute output from the controller */
USTRUCT(BlueprintType)
struct FAnimGenControllerAttributeOutput
{
	GENERATED_BODY()

public:

	/** Auto-Encoder Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName AutoEncoderAttributeName = NAME_None;

	/** Output if the provided attribute is active or not */
	UPROPERTY(EditAnywhere, Category = "Entry")
	bool bOutputAutoEncoderAttributeActive = false;

	/** Output Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName OutputAttributeName = NAME_None;

	/** Output Attribute bone */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName OutputBoneName = TEXT("root");
};

/** Specifies a bone transform output from the controller */
USTRUCT(BlueprintType)
struct FAnimGenControllerBoneOutput
{
	GENERATED_BODY()

public:

	/** Auto-Encoder Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName AutoEncoderAttributeName = NAME_None;

	/** Output Bone name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName OutputBoneName = NAME_None;
};

/** Specifies an Anim Notify output from the controller */
USTRUCT(BlueprintType)
struct FAnimGenControllerAnimNotifyOutput
{
	GENERATED_BODY()

public:

	/** Auto-Encoder Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName AutoEncoderAttributeName = NAME_None;

	/** Output Anim Notify */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimNotify> AnimNotify;

	/** Amount of time before the event to output the notify */
	UPROPERTY(EditAnywhere, Category = "Entry", meta=(ClampMin="-0.9", UIMin="-0.9", ClampMax="0.9", UIMax="0.9", ForceUnits="s"))
	float Apprehension = 0.0f;
};

/** Specifies an Anim Notify State output from the controller */
USTRUCT(BlueprintType)
struct FAnimGenControllerAnimNotifyStateOutput
{
	GENERATED_BODY()

public:

	/** Auto-Encoder Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName AutoEncoderAttributeName = NAME_None;

	/** Output if the provided attribute is active or not */
	UPROPERTY(EditAnywhere, Category = "Entry")
	bool bOutputAutoEncoderAttributeActive = false;

	/** Output Anim Notify State */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimNotifyState> AnimNotifyState;
};

/**
 * AnimNode that can be used to generate animation using an AnimGenController.
 */
USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_AnimGenController : public FAnimNode_Base
{
	GENERATED_BODY()

	/** Controller asset used by the node */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinShownByDefault))
	TObjectPtr<UAnimGenController> Controller;

	/** Control Object taken as input */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinShownByDefault))
	FAnimGenControlObject ControlObject;

	/** Control Object Element from Control Object to use as input */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinShownByDefault))
	FAnimGenControlObjectElement ControlObjectElement;

	/** Which LOD level to use */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinShownByDefault, ClampMin = "0", UIMin = "0", ClampMax = "2", UIMax = "2"))
	int32 LODLevel = 0;

public:

	/** Random Seed to initialize sampling with */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, ClampMin = "0", UIMin = "0"))
	int32 Seed = 1234;

	/** If to reset the node when it becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bResetOnBecomingRelevant = true;

	/** The period at which to evaluate the controller in frames. Increasing this can improve performance at the cost of some responsiveness. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, ClampMin = "0.0", UIMin = "0.0", ClampMax = "8.0", UIMax = "8.0"))
	float EvaluationPeriodInFrames = 0.0f;

	/** If to force an evaluation of the controller this frame */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault))
	bool bForceEvaluation = false;

public:

	/** Minimum Smoothing time to use - mixes the predicted pose with an integration of the predicted velocity. */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (PinHiddenByDefault, ClampMin = "0.0", UIMin = "0.0"))
	float MinPoseSmoothingTime = 0.1f;

	/** Maximum Smoothing time to use - mixes the predicted pose with an integration of the predicted velocity. */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (PinHiddenByDefault, ClampMin = "0.0", UIMin = "0.0"))
	float MaxPoseSmoothingTime = 0.25f;

	/** Smoothing to apply to predicted attributes */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (PinHiddenByDefault, ClampMin = "0.0", UIMin = "0.0"))
	float AttributeSmoothingTime = 0.0f;

public:

	/** Array of additional curve outputs */
	UPROPERTY(EditAnywhere, Category = "Additional Outputs")
	TArray<FAnimGenControllerCurveOutput> CurveOutputs;

	/** Array of additional attribute outputs */
	UPROPERTY(EditAnywhere, Category = "Additional Outputs")
	TArray<FAnimGenControllerAttributeOutput> AttributeOutputs;

	/** Array of additional bone outputs */
	UPROPERTY(EditAnywhere, Category = "Additional Outputs")
	TArray<FAnimGenControllerBoneOutput> BoneOutputs;

	/** Array of additional Anim Notify outputs */
	UPROPERTY(EditAnywhere, Category = "Additional Outputs")
	TArray<FAnimGenControllerAnimNotifyOutput> AnimNotifyOutputs;

	/** Array of additional Anim Notify State outputs */
	UPROPERTY(EditAnywhere, Category = "Additional Outputs")
	TArray<FAnimGenControllerAnimNotifyStateOutput> AnimNotifyStateOutputs;

public:

	/** If to clamp to zero root velocities that are below a given threshold */
	UPROPERTY(EditAnywhere, Category = "Root Velocity Deadzone")
	bool bApplyRootVelocityDeadzone = true;

	/** Minimum linear velocity under which it will be clamped to zero when clamping is enabled */
	UPROPERTY(EditAnywhere, Category = "Root Velocity Deadzone", meta = (EditCondition = "bApplyRootVelocityDeadzone", HideEditConditionToggle, ClampMin = "0.0", UIMin = "0.0", ForceUnits="cm/s"))
	float RootLinearVelocityDeadzone = 1.0f;

	/** Minimum angular velocity under which it will be clamped to zero when clamping is enabled */
	UPROPERTY(EditAnywhere, Category = "Root Velocity Deadzone", meta = (EditCondition = "bApplyRootVelocityDeadzone", HideEditConditionToggle, ClampMin = "0.0", UIMin = "0.0", ForceUnits = "deg/s"))
	float RootAngularVelocityDeadzone = 0.1f;

public:

	// FAnimNode_Base interface
	virtual UE_API void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	virtual UE_API void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	virtual UE_API void Evaluate_AnyThread(FPoseContext& Output) override;
	// End of FAnimNode_Base interface

private:

	/** Update Counter for detecting being relevant */
	FGraphTraversalCounter UpdateCounter;

	/** Random state used for sampling initial noise */
	uint32 RandomState = 1234;

	/** Current active controller object */
	UPROPERTY(Transient)
	TObjectPtr<UAnimGenController> ActiveController = nullptr;

	/** Internal buffer of AnimNotifyEvents */
	UPROPERTY(Transient)
	TArray<FAnimNotifyEvent> AnimNotifyEvents;

	/** Internal buffer of AnimNotifyEventReferences */
	UPROPERTY(Transient)
	TArray<FAnimNotifyEventReference> AnimNotifyEventReferences;

	// Control Variables

	bool bControlsValid = false;
	int32 ControlSchemaHash = 0;
	int32 ControlVectorSize = 0;
	int32 EncodedControlVectorSize = 0;
	int32 ControlDistributionVectorSize = 0;
	TLearningArray<2, float> ControlVector;
	TLearningArray<2, float> EncodedControlVector;

	// Pose Variables

	float PoseDeltaTime = 0.0f;
	float BlockTime = 0.0f;
	bool bPoseStateRequiresReset = true;
	int32 EncodedPoseVectorSize = 0;
	int32 PoseVectorSize = 0;
	TLearningArray<2, float> EncodedPoseVector;
	TLearningArray<2, float> EncodedPoseVectorBlock;
	TLearningArray<2, float> UnnormalizedEncodedPoseVector;
	TLearningArray<2, float> PoseVector;
	UE::AnimDatabase::FPoseData CurrPoseData;
	UE::AnimDatabase::FPoseData PrevPoseData;
	UE::AnimDatabase::FPoseData OutputPoseData;
	TArray<TTuple<FName, float>> OutputCurves;

	// Network Instances

	TLearningArray<2, float> DenoiserInputVector;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> ControlEncoderInference;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> LOD0Inference;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> LOD1Inference;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> LOD2Inference;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> EncoderInference;
	TSharedPtr<UE::Learning::FNeuralNetworkInference> DecoderInference;

	// Root Motion Provider
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = nullptr;
};

#undef UE_API