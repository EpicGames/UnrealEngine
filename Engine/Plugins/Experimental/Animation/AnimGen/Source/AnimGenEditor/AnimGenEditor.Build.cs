// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimGenEditor : ModuleRules
{
	public AnimGenEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AnimDatabase",
				"AnimDatabaseEditor",
				"AnimGen",
				"Learning",
				"LearningTraining",
				"AnimGraph",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"LevelEditor",
				"DetailCustomizations",
				"EditorStyle",
				"Projects",
				"AssetDefinition",
				"EditorWidgets",
				"PropertyEditor",
				"KismetWidgets",
				"SequencerWidgets",
				"ToolWidgets",
				"AdvancedPreviewScene",
				"GraphEditor",
				"ClassViewer",
				"RenderCore",
				"BlueprintGraph",
				"Learning",
				"LearningTraining",
				"Json",
				"JsonUtilities",
				"NNE",
				"NNERuntimeBasicCpu",
				"Blutility",
				"DrawDebugLibrary",
				"PythonScriptPlugin",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
