// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditor.h"

#include "AnimGenEditorAutoEncoderMode.h"
#include "AnimGenEditorControllerMode.h"

#include "AnimGenEditorAutoEncoderToolkit.h"
#include "AnimGenEditorControllerToolkit.h"

#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FAnimGenEditorModule"

void FAnimGenEditorModule::StartupModule()
{
	// Register Ed Mode used to interact with the preview scene
	FEditorModeRegistry::Get().RegisterMode<UE::AnimGen::Editor::FAutoEncoderMode>(UE::AnimGen::Editor::FAutoEncoderMode::EditorModeId, LOCTEXT("AnimGenEditorAutoEncoderModeName", "AnimGenAutoEncoder"));
	FEditorModeRegistry::Get().RegisterMode<UE::AnimGen::Editor::FControllerMode>(UE::AnimGen::Editor::FControllerMode::EditorModeId, LOCTEXT("AnimGenEditorControllerModeName", "AnimGenController"));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("AnimGenAutoEncoder", FOnGetDetailCustomizationInstance::CreateStatic(&UE::AnimGen::Editor::FAutoEncoderDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("AnimGenController", FOnGetDetailCustomizationInstance::CreateStatic(&UE::AnimGen::Editor::FControllerDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("AnimGenAutoEncoderTrainingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&UE::AnimGen::Editor::FAutoEncoderTrainingSettingsDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("AnimGenControllerTrainingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&UE::AnimGen::Editor::FControllerTrainingSettingsDetails::MakeInstance));
}

void FAnimGenEditorModule::ShutdownModule()
{
	// Unregister Ed Mode
	FEditorModeRegistry::Get().UnregisterMode(UE::AnimGen::Editor::FAutoEncoderMode::EditorModeId);
	FEditorModeRegistry::Get().UnregisterMode(UE::AnimGen::Editor::FControllerMode::EditorModeId);

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("AnimGenAutoEncoder");
		PropertyModule.UnregisterCustomClassLayout("AnimGenController");
		PropertyModule.UnregisterCustomClassLayout("AnimGenAutoEncoderTrainingSettings");
		PropertyModule.UnregisterCustomClassLayout("AnimGenControllerTrainingSettings");
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAnimGenEditorModule, AnimGenEditor)