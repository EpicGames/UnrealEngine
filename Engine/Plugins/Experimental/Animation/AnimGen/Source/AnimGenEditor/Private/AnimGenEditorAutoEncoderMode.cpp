// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditorAutoEncoderMode.h"
#include "AnimGenEditorAutoEncoderToolkit.h"

#include "AnimDatabaseEditorViewportClient.h"
#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorTimeline.h"

#include "AnimGenAutoEncoder.h"

#include "AnimDatabase.h"
#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameAttribute.h"

#include "LearningNeuralNetwork.h"
#include "LearningRandom.h"
#include "LearningFrameSet.h"
#include "LearningFrameAttribute.h"

#include "GameFramework/Character.h"
#include "Components/PoseableMeshComponent.h"
#include "Components/CapsuleComponent.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "Async/ParallelFor.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "ITransportControl.h"
#include "DrawDebugLibrary.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorAutoEncoderMode"

namespace UE::AnimGen::Editor
{
	namespace AutoEncoder::Private
	{
		static inline FVector GetRangeOffset(
			const int32 CharacterIdx,
			const int32 CharacterNum,
			const float Spacing,
			const bool bOffsetRanges,
			const bool bOffsetRangesInGrid)
		{
			const int32 GridRows = FMath::CeilToInt(FMath::Sqrt((float)CharacterNum));

			return
				bOffsetRanges ?
				bOffsetRangesInGrid ?
				FVector(
					Spacing * (CharacterIdx % GridRows) - 0.5f * Spacing * (GridRows - 1),
					Spacing * (CharacterIdx / GridRows) - 0.5f * Spacing * (GridRows - 1), 0.0f) :
				FVector(
					Spacing * CharacterIdx - 0.5f * Spacing * (CharacterNum - 1), 0.0f, 0.0f) :
				FVector::ZeroVector;
		}

		static inline void AdjustTransformRootTransform(
			FVector& InOutLocation,
			FQuat4f& InOutRotation,
			FVector3f& InOutLinearVelocity,
			FVector3f& InOutAngularVelocity,
			const FVector Offset,
			const FTransform RelativeTransform,
			const bool bDisableRootTranslation,
			const bool bDisableRootRotation)
		{
			const FTransform RootOffsetTransform = FTransform(FQuat::Identity, Offset, FVector::OneVector);
			const FTransform RootAdjustmenTransform = RelativeTransform.Inverse() * RootOffsetTransform;

			if (bDisableRootTranslation && bDisableRootRotation)
			{
				const FQuat4f RootRotationOffset = InOutRotation;
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = FQuat4f::Identity;
				InOutLinearVelocity = RootRotationOffset.UnrotateVector(InOutLinearVelocity);
				InOutAngularVelocity = RootRotationOffset.UnrotateVector(InOutAngularVelocity);
			}
			else if (bDisableRootTranslation)
			{
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
			else
			{
				InOutLocation = RootAdjustmenTransform.TransformPosition(InOutLocation);
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
		}
	}

	const FEditorModeID FAutoEncoderMode::EditorModeId = TEXT("AnimGenEditorAutoEncoderMode");

	FAutoEncoderMode::FAutoEncoderMode() = default;
	FAutoEncoderMode::~FAutoEncoderMode() = default;

	void FAutoEncoderMode::Tick(FEditorViewportClient* EditorViewportClient, float DeltaTime)
	{
		FEdMode::Tick(EditorViewportClient, DeltaTime);

		// Get the viewport client

		AnimDatabase::Editor::FViewportClient* ViewportClient = static_cast<AnimDatabase::Editor::FViewportClient*>(EditorViewportClient);
		check(ViewportClient);
		ViewportClient->ClearWarningMessage();

		// Grab the Editor Toolkit

		TSharedPtr<FAutoEncoderToolkit> EditorToolKit = StaticCastSharedPtr<FAutoEncoderToolkit>(ViewportClient->GetAssetEditorToolkit().Pin());
		if (!EditorToolKit.IsValid()) { return; }

		WeakToolkit = EditorToolKit.ToWeakPtr();

		TSharedPtr<UE::AnimDatabase::Editor::FPreviewScene> PreviewScene = ViewportClient->GetPreviewScene().Pin();
		if (!PreviewScene.IsValid()) { return; }

		// Get the autoencoder

		UAnimGenAutoEncoder* AutoEncoder = EditorToolKit->AutoEncoder;
		check(AutoEncoder);

		// Update Query

		AutoEncoder->TrainingSettings->RangeIndentifierColorSeed = AutoEncoder->ViewportSettings->RangeIdentifierColorSeed;
		const bool bQueryUpdated = AutoEncoder->TrainingSettings->Update();

		if (bQueryUpdated)
		{
			ActiveRanges.Reset();
			EditorToolKit->TrainingSettingsWidget->ForceRefresh();
		}

		// Check we have a query database with matching skeleton

		UAnimDatabase* QueryDatabase = AutoEncoder->TrainingSettings->Database.Get();
		if (!QueryDatabase)
		{
			ViewportClient->SetWarningMessage(
				AutoEncoder->TrainingSettings->Database.IsNull() ? 
				LOCTEXT("NoQuery", "Warning: No Training Database Provided.") :
				LOCTEXT("LoadingDatabase", "Loading Training Database..."));
			ActiveRanges.Reset();
			return;
		}

		// Check if the trained database content has changed and if so force refresh of details panel to check if warning needs to be displayed

		const int32 SettingsContentHash = UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(QueryDatabase, AutoEncoder->TrainingSettings->QueryRanges);

		if (CurrentContentHash != SettingsContentHash)
		{
			CurrentContentHash = SettingsContentHash;
			EditorToolKit->EditingAssetWidget->ForceRefresh();
		}

		// Add the required number of characters to the scene

		while (PreviewScene->GetCharacterNum() < MaxCharacterNum)
		{
			const int32 CharacterIdx = PreviewScene->AddCharacter();
			PreviewScene->SetCharacterVisibility(CharacterIdx, false);
		}

		// Update the Desired Skeletal Meshes

		USkinnedAsset* DesiredSkeletalMesh = QueryDatabase->Skeleton ? QueryDatabase->Skeleton->GetPreviewMesh() : nullptr;
		DesiredSkeletalMesh = AutoEncoder->ViewportSettings->PreviewMesh ? AutoEncoder->ViewportSettings->PreviewMesh.Get() : DesiredSkeletalMesh;

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			USkinnedAsset* CurrentSkeletalMesh = PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset();

			if (DesiredSkeletalMesh && DesiredSkeletalMesh != CurrentSkeletalMesh)
			{
				PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetSkinnedAssetAndUpdate(DesiredSkeletalMesh, true);
			}
		}

		// Compute all range trajectories

		if (bQueryUpdated || RangeTrajectoryLocations.IsEmpty())
		{
			const int32 QueryRangeNum = AutoEncoder->TrainingSettings->QueryRanges.IsValid() ?
				AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetTotalRangeNum() : 0;

			RangeTrajectoryLocations.SetNum(QueryRangeNum);
			RangeTrajectoryRotations.SetNum(QueryRangeNum);

			if (AutoEncoder->TrainingSettings->QueryRanges.IsValid())
			{
				QueryDatabase->WaitForCompressionOnAnimSequencesFromArrayView(AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntrySequences());
			}

			ParallelFor(QueryRangeNum, [this, QueryDatabase, AutoEncoder](int32 RangeIdx)
				{
					int32 RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
					const bool bFound = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->FindTotalRange(RangeEntry, RangeRange, RangeIdx);
					check(bFound);

					const int32 RangeFrameNum = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntryRangeLength(RangeEntry, RangeRange);
					const int32 RangeSequence = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntrySequence(RangeEntry);
					const int32 RangeStart = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntryRangeStart(RangeEntry, RangeRange);

					RangeTrajectoryLocations[RangeIdx].SetNumUninitialized({ RangeFrameNum });
					RangeTrajectoryRotations[RangeIdx].SetNumUninitialized({ RangeFrameNum });

					QueryDatabase->GetRootLocation(RangeTrajectoryLocations[RangeIdx], RangeSequence, RangeStart);
					QueryDatabase->GetRootRotation(RangeTrajectoryRotations[RangeIdx], RangeSequence, RangeStart);
				});
		}

		// Adjust Number of visible characters in the scene

		const int32 CharacterNum = FMath::Min(AutoEncoder->TrainingSettings->SelectedRanges.Num(), MaxCharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			PreviewScene->SetCharacterVisibility(CharacterIdx, CharacterIdx < CharacterNum);
		}

		// If selection has changed adjust view ranges

		bool bSelectedRangesUpdated = false;

		if (bQueryUpdated || ActiveRanges != AutoEncoder->TrainingSettings->SelectedRanges)
		{
			bSelectedRangesUpdated = true;
			ActiveRanges = AutoEncoder->TrainingSettings->SelectedRanges;

			// Add error message if too many ranges selected

			if (ActiveRanges.Num() > MaxCharacterNum)
			{
				ViewportClient->SetWarningMessage(FText::Format(
					LOCTEXT("AnimGenTooManyRangesWarning", "Warning: Rendering characters for {0} for {1} selected ranges."),
					FText::AsNumber(MaxCharacterNum),
					FText::AsNumber(ActiveRanges.Num())));
			}

			// Adjust the timeline for the newly selected ranges

			if (ActiveRanges.Num() == 0)
			{
				EditorToolKit->TimelineModel->SetTimelineFrameRate(QueryDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
			else if (ActiveRanges.Num() == 1)
			{
				const int32 RangeStart = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRanges[0]];
				const int32 RangeLength = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRanges[0]];

				EditorToolKit->TimelineModel->SetTimelineFrameRate(QueryDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(RangeStart));
			}
			else
			{
				int32 RangeLength = 1;
				for (const int32 ActiveRangeIdx : ActiveRanges)
				{
					RangeLength = FMath::Max(AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRangeIdx], RangeLength);
				}

				EditorToolKit->TimelineModel->SetTimelineFrameRate(QueryDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
		}

		// Update Character Range Data

		CharacterSequenceIndices.SetNumUninitialized(CharacterNum);
		CharacterRangeTimes.SetNumUninitialized(CharacterNum);
		CharacterRangeStarts.SetNumUninitialized(CharacterNum);
		CharacterRangeLengths.SetNumUninitialized(CharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			int32 TestEntryIdx, TestEntryRangeIdx = INDEX_NONE;
			const bool bFound = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->FindTotalRange(TestEntryIdx, TestEntryRangeIdx, ActiveRanges[CharacterIdx]);
			check(bFound);

			CharacterSequenceIndices[CharacterIdx] = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntrySequence(TestEntryIdx);
			CharacterRangeStarts[CharacterIdx] = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRanges[CharacterIdx]];
			CharacterRangeLengths[CharacterIdx] = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRanges[CharacterIdx]];

			CharacterRangeTimes[CharacterIdx] = 0.0;
			if (ActiveRanges.Num() == 1)
			{
				CharacterRangeTimes[CharacterIdx] = EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() / QueryDatabase->GetFrameRate().AsDecimal();
			}
			else
			{
				CharacterRangeTimes[CharacterIdx] = FMath::Clamp(
					EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() + (float)CharacterRangeStarts[CharacterIdx],
					(float)CharacterRangeStarts[CharacterIdx], 
					(float)(CharacterRangeStarts[CharacterIdx] + CharacterRangeLengths[CharacterIdx] - 1)) / QueryDatabase->GetFrameRate().AsDecimal();
			}

			CharacterRangeTimes[CharacterIdx] = FMath::Clamp(CharacterRangeTimes[CharacterIdx],
				0.0f, FMath::Max(QueryDatabase->GetSequenceDuration(CharacterSequenceIndices[CharacterIdx]), 0.0f));

			check(QueryDatabase->GetAnimSequence(CharacterSequenceIndices[CharacterIdx]))
		}

		// Is the auto-encoder valid (trained?)

		bAutoEncoderValid = AutoEncoder->IsValid();

		// If Auto-Encoder is valid re-build frame attributes

		if (bAutoEncoderValid && (bQueryUpdated || !bFrameAttributesGenerated))
		{
			const int32 AttributeNum = AutoEncoder->TrainedFrameAttributes.Num();
			FrameAttributeObjects.SetNum(AttributeNum);
			FrameAttributeTypes.SetNum(AttributeNum);
			FrameAttributeNames.SetNum(AttributeNum);
			bFrameAttributesGenerated = true;

			if (AttributeNum > 0)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingFrameAttributes", "Rebuilding AutoEncoder Frame Attributes in Database"));
				SlowTask.MakeDialog();

				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					FrameAttributeObjects[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(
						QueryDatabase, AutoEncoder->TrainingSettings->QueryRanges, AutoEncoder->TrainedFrameAttributes[AttributeIdx].FrameAttribute);

					if (!FrameAttributeObjects[AttributeIdx].IsValid())
					{
						FrameAttributeObjects[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeNullFrameAttribute(AutoEncoder->TrainingSettings->QueryRanges);
					}

					FrameAttributeTypes[AttributeIdx] = FrameAttributeObjects[AttributeIdx].Type;
					FrameAttributeNames[AttributeIdx] = AutoEncoder->TrainedFrameAttributes[AttributeIdx].Name;
				}
			}
		}
		else if (!bAutoEncoderValid)
		{
			FrameAttributeObjects.Reset();
			FrameAttributeTypes.Reset();
			FrameAttributeNames.Reset();
			bFrameAttributesGenerated = false;
		}

		if (bAutoEncoderValid && AutoEncoder->AttributeTypes != FrameAttributeTypes)
		{
			ViewportClient->SetWarningMessage(LOCTEXT("InvalidFrameAttributes", "Warning: Frame Attributes don't match AutoEncoder."));
			bAutoEncoderValid = false;
		}

		if (bAutoEncoderValid)
		{
			const int32 AutoEncoderBoneNum = AutoEncoder->GetBoneNum();
			for (int32 BoneIdx = 0; BoneIdx < AutoEncoderBoneNum; BoneIdx++)
			{
				const FName BoneName = AutoEncoder->GetBoneName(BoneIdx);

				if (QueryDatabase->FindBoneIndex(BoneName) == INDEX_NONE)
				{
					ViewportClient->SetWarningMessage(FText::Format(
						LOCTEXT("BoneMissing", "Warning: Bone {0} from AutoEncoder no longer in database."),
						FText::FromName(BoneName)));

					bAutoEncoderValid = false;
					break;
				}
			}
		}

		// If selection has changed or additional frames/frame ranges has changed update tracks

		if (bSelectedRangesUpdated)
		{
			EditorToolKit->TracksModel->ResetTracks();

			TArray<TSubclassOf<UAnimNotify>> TrackAnimNotifies;
			TArray<TSubclassOf<UAnimNotifyState>> TrackAnimNotifyStates;
			TArray<FName> TrackAnimCurves;
			TArray<FName, TInlineAllocator<32>> TrackFrames;
			TArray<FName, TInlineAllocator<32>> TrackFrameRanges;
			TArray<FName, TInlineAllocator<32>> TrackFrameAttributes;
			TArray<FAnimDatabaseFrames, TInlineAllocator<32>> TrackFramesStruct;
			TArray<FAnimDatabaseFrameRanges, TInlineAllocator<32>> TrackFrameRangesStruct;
			TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> TrackFrameAttributesStruct;

			for (const int32 ActiveRangeIdx : ActiveRanges)
			{
				const int32 ViewOffset = ActiveRanges.Num() > 1 ? AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRangeIdx] : 0;
				int32 EntryIdx = INDEX_NONE, EntryRangeIdx = INDEX_NONE;
				AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->FindTotalRange(EntryIdx, EntryRangeIdx, ActiveRangeIdx);
				const int32 SequenceIdx = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
				const int32 RangeStart = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, EntryRangeIdx);
				const int32 RangeLength = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, EntryRangeIdx);

				const FAnimDatabaseFrameRanges SequenceRanges =
					ActiveRanges.Num() > 1 ?
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceRange(QueryDatabase, SequenceIdx, RangeStart, RangeLength) :
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceIndex(QueryDatabase, SequenceIdx);

				// Make Frames

				UAnimDatabaseFrameRangesLibrary::FindAnimNotifyClassesInFrameRanges(
					TrackAnimNotifies,
					QueryDatabase,
					SequenceRanges);

				const int32 AnimNotifyNum = TrackAnimNotifies.Num();
				TrackFrames.Reset();
				TrackFramesStruct.Reset();
				for (int32 AnimNotifyIdx = 0; AnimNotifyIdx < AnimNotifyNum; AnimNotifyIdx++)
				{
					const FAnimDatabaseFrames NotifyFrames = UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(QueryDatabase, SequenceRanges, TrackAnimNotifies[AnimNotifyIdx]);

					if (NotifyFrames.IsValid() && !NotifyFrames.FrameSet->IsEmpty())
					{
						TrackFrames.Add(TrackAnimNotifies[AnimNotifyIdx]->GetFName());
						TrackFramesStruct.Add(NotifyFrames);
					}
				}

				// Make Frame Ranges

				UAnimDatabaseFrameRangesLibrary::FindAnimNotifyStateClassesInFrameRanges(
					TrackAnimNotifyStates,
					QueryDatabase,
					SequenceRanges);

				const int32 AnimNotifyStateNum = TrackAnimNotifyStates.Num();
				TrackFrameRanges.Reset();
				TrackFrameRangesStruct.Reset();
				for (int32 AnimNotifyStateIdx = 0; AnimNotifyStateIdx < AnimNotifyStateNum; AnimNotifyStateIdx++)
				{
					const FAnimDatabaseFrameRanges NotifyStateFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(QueryDatabase, SequenceRanges, TrackAnimNotifyStates[AnimNotifyStateIdx]);

					if (NotifyStateFrameRanges.IsValid() && !NotifyStateFrameRanges.FrameRangeSet->IsEmpty())
					{
						TrackFrameRanges.Add(TrackAnimNotifyStates[AnimNotifyStateIdx]->GetFName());
						TrackFrameRangesStruct.Add(NotifyStateFrameRanges);
					}
				}

				// Make Frame Attributes

				UAnimDatabaseFrameRangesLibrary::FindCurvesInFrameRanges(
					TrackAnimCurves,
					QueryDatabase,
					SequenceRanges);

				const int32 CurveNum = TrackAnimCurves.Num();
				TrackFrameAttributes.Reset();
				TrackFrameAttributesStruct.Reset();
				for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
				{
					const FAnimDatabaseFrameAttribute CurveAttribute = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurveWhenActive(QueryDatabase, SequenceRanges, TrackAnimCurves[CurveIdx]);

					if (CurveAttribute.IsValid() && !CurveAttribute.FrameAttribute->IsEmpty())
					{
						TrackFrameAttributes.Add(TrackAnimCurves[CurveIdx]);
						TrackFrameAttributesStruct.Add(CurveAttribute);
					}
				}

				const int32 FrameAttributeNum = FrameAttributeObjects.Num();
				for (int32 TrainedIdx = 0; TrainedIdx < FrameAttributeNum; TrainedIdx++)
				{
					if (FrameAttributeObjects[TrainedIdx].IsValid())
					{
						const FAnimDatabaseFrameAttribute SequenceTrainedFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(FrameAttributeObjects[TrainedIdx], SequenceRanges);

						if (SequenceTrainedFrameAttribute.IsValid() && !SequenceTrainedFrameAttribute.FrameAttribute->IsEmpty())
						{
							TrackFrameAttributes.Add(FrameAttributeNames[TrainedIdx]);
							TrackFrameAttributesStruct.Add(SequenceTrainedFrameAttribute);
						}
					}
				}

				// Add Track

				EditorToolKit->TracksModel->AddTrack(
					AutoEncoder->TrainingSettings->QueryEntries[ActiveRangeIdx]->Sequence,
					AutoEncoder->TrainingSettings->QueryEntries[ActiveRangeIdx]->Color,
					TrackFrames,
					TrackFrameRanges,
					TrackFrameAttributes,
					TrackFramesStruct,
					TrackFrameRangesStruct,
					TrackFrameAttributesStruct,
					ViewOffset);
			}

			// Reconstruct the timeline widget for the current selection
			EditorToolKit->ReconstructTimelineWidget();
		}

		// Resize Pose State and re-create Network Inference Instances

		if (!OriginalPoseState.IsValid() || !ReconstructedPoseState.IsValid())
		{
			OriginalPoseState = UAnimDatabasePoseStateLibrary::MakePoseState();
			ReconstructedPoseState = UAnimDatabasePoseStateLibrary::MakePoseState();
		}

		// Resize the pose data

		if (!bAutoEncoderValid)
		{
			QueryDatabase->GetBoneParents(BoneParents);
			OriginalPoseState.CurrPoseData->Resize(CharacterNum, QueryDatabase->GetBoneNum(), {}, {});
			OriginalPoseState.PoseGlobalBoneData->Resize(CharacterNum, QueryDatabase->GetBoneNum());
			ReconstructedPoseState.CurrPoseData->Empty();
			ReconstructedPoseState.PoseGlobalBoneData->Empty();
			OriginalPoseVector.Empty();
			EncodedVector.Empty();
			ReconstructedPoseVector.Empty();
		}
		else
		{
			const int32 PoseVectorSize = AutoEncoder->GetPoseVectorSize();
			const int32 EncodingSize = AutoEncoder->GetEncodingSize();

			BoneParents = AutoEncoder->GetBoneParents();
			OriginalPoseState.CurrPoseData->Resize(CharacterNum, AutoEncoder->GetBoneNum(), FrameAttributeTypes, FrameAttributeNames);
			OriginalPoseState.PoseGlobalBoneData->Resize(CharacterNum, AutoEncoder->GetBoneNum());
			ReconstructedPoseState.CurrPoseData->Resize(CharacterNum, AutoEncoder->GetBoneNum(), FrameAttributeTypes, FrameAttributeNames);
			ReconstructedPoseState.PoseGlobalBoneData->Resize(CharacterNum, AutoEncoder->GetBoneNum());
			OriginalPoseVector.SetNumUninitialized({ CharacterNum, PoseVectorSize });
			EncodedVector.SetNumUninitialized({ CharacterNum, EncodingSize });
			ReconstructedPoseVector.SetNumUninitialized({ CharacterNum, PoseVectorSize });
		}

		// Invalidate PCA Visualization

		if (bAutoEncoderValid && (bQueryUpdated || (PCAEncoder.IsValid() && PCAEncoder->DimensionNum() != AutoEncoder->GetEncodingSize())))
		{
			PCAEncoder.Reset();
			PCAEncodedPoses.Empty();
		}

		// Rebuild PCA Encoding Visualization

		if (bAutoEncoderValid && AutoEncoder->ViewportSettings->bForceRebuildEncodingVisualization)
		{
			AutoEncoder->ViewportSettings->bForceRebuildEncodingVisualization = false;

			FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingEncoding", "Rebuilding Encoding Visualization"));
			SlowTask.MakeDialog();

			const int32 QueryTotalFrameNum = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetTotalFrameNum();
			const int32 QueryTotalRangeNum = AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetTotalRangeNum();
			const int32 PoseEncodingSize = AutoEncoder->GetEncodingSize();
			const int32 PoseVectorSize = AutoEncoder->GetPoseVectorSize();

			TLearningArray<1, int32> QueryRangeSequenceIndices;
			QueryRangeSequenceIndices.SetNumUninitialized({ QueryTotalRangeNum });
			UE::Learning::FrameRangeSet::AllRangeSequences(QueryRangeSequenceIndices, *AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet);

			QueryDatabase->WaitForCompressionOnAnimSequencesFromArrayView(QueryRangeSequenceIndices);

			AnimDatabase::FPoseData QueryPoseData;
			QueryPoseData.Resize(QueryTotalFrameNum, AutoEncoder->AutoEncodedRequiredBoneIndices.Num(), FrameAttributeTypes, FrameAttributeNames);

			ParallelFor(QueryTotalRangeNum, [
				this,
					AutoEncoder,
					QueryDatabase,
					&QueryRangeSequenceIndices,
					&QueryPoseData](int32 RangeIdx)
				{
					QueryDatabase->GetPoseSubsetData(
						QueryPoseData.Slice(
							AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeOffsets()[RangeIdx],
							AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx]),
						QueryRangeSequenceIndices[RangeIdx],
						AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet->GetAllRangeStarts()[RangeIdx],
						AutoEncoder->AutoEncodedRequiredBoneIndices,
						FrameAttributeObjects);
				});

			TLearningArray<2, float> QueryPoseVectors;
			QueryPoseVectors.SetNumUninitialized({ QueryTotalFrameNum, PoseVectorSize });

			UE::Learning::SlicedParallelFor(QueryTotalFrameNum, 512, [AutoEncoder, &QueryPoseVectors, &QueryPoseData](int32 SliceStart, int32 SliceLength)
				{
					AutoEncoder->ToPoseVectors(
						QueryPoseVectors.Slice(SliceStart, SliceLength),
						QueryPoseData.ConstSlice(SliceStart, SliceLength),
						AutoEncoder->AutoEncodedRequiredBoneIndices);

					AutoEncoder->NormalizePoseVectors(QueryPoseVectors.Slice(SliceStart, SliceLength));
				});

			QueryPoseData.Empty();

			TLearningArray<2, float> QueryEncodedVectors;
			QueryEncodedVectors.SetNumUninitialized({ QueryTotalFrameNum, PoseEncodingSize });

			TSharedPtr<UE::Learning::FNeuralNetworkInference> BatchInference = AutoEncoder->EncoderNetwork->GetNetwork()->CreateInferenceObject(QueryTotalFrameNum);
			BatchInference->Evaluate(QueryEncodedVectors, QueryPoseVectors);
			BatchInference.Reset();
			QueryPoseVectors.Empty();

			// Subtract mean and normalize scale

			TLearningArray<1, float> QueryStd;
			QueryStd.SetNumUninitialized({ PoseEncodingSize });
			PCAEncoderMean.SetNumUninitialized({ PoseEncodingSize });
			UE::AnimDatabase::Math::ComputeMeanStd(PCAEncoderMean, QueryStd, QueryEncodedVectors);
			UE::AnimDatabase::Math::ComputeMean(PCAEncoderStd, QueryStd);

			for (int32 FrameIdx = 0; FrameIdx < QueryTotalFrameNum; FrameIdx++)
			{
				for (int32 DimIdx = 0; DimIdx < PoseEncodingSize; DimIdx++)
				{
					QueryEncodedVectors[FrameIdx][DimIdx] = (QueryEncodedVectors[FrameIdx][DimIdx] - PCAEncoderMean[DimIdx]) / FMath::Max(PCAEncoderStd, UE_SMALL_NUMBER);
				}
			}

			Learning::FPCASettings PCASettings;
			PCASettings.bStableComputation = true;
			PCASettings.MaximumDimensions = 3;

			PCAEncoder = MakeShared<Learning::FPCAEncoder>();
			PCAEncoder->Fit(QueryEncodedVectors, PCASettings);

			PCAEncodedPoses.SetNumUninitialized({ QueryTotalFrameNum, 3 });
			PCAEncoder->Transform(PCAEncodedPoses, QueryEncodedVectors);
			QueryEncodedVectors.Empty();
		}

		// Update Viewport Settings state based on if we can draw or not

		if (bAutoEncoderValid && AutoEncoder->ViewportSettings->bCanDrawEncodingVisualization != PCAEncoder.IsValid())
		{
			AutoEncoder->ViewportSettings->bCanDrawEncodingVisualization = PCAEncoder.IsValid();
			EditorToolKit->ViewportSettingsWidget->ForceRefresh();
		}

		// Evaluate Pose and attributes

		if (!bAutoEncoderValid)
		{
			QueryDatabase->WaitForCompressionOnAnimSequences(CharacterSequenceIndices);

			QueryDatabase->SamplePoseData(
				OriginalPoseState.CurrPoseData->View(),
				CharacterSequenceIndices,
				CharacterRangeTimes);
		}
		else
		{
			QueryDatabase->WaitForCompressionOnAnimSequences(CharacterSequenceIndices);

			TArray<int32> AutoEncoderPoseDatabaseIndices;
			AutoEncoderPoseDatabaseIndices.SetNumUninitialized(AutoEncoder->GetBoneNum());
			QueryDatabase->FindBoneIndicesFromArrayViews(AutoEncoderPoseDatabaseIndices, AutoEncoder->GetBoneNames());

			QueryDatabase->SamplePoseSubsetData(
				OriginalPoseState.CurrPoseData->View(),
				CharacterSequenceIndices,
				CharacterRangeTimes,
				AutoEncoderPoseDatabaseIndices,
				FrameAttributeObjects);
		}

		if (AutoEncoder->ViewportSettings->bRangesStartAtOrigin)
		{
			for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
			{
				const FVector RootOffset = AutoEncoder::Private::GetRangeOffset(
					CharacterIdx, CharacterNum,
					AutoEncoder->ViewportSettings->RangeOffsetSpacing,
					AutoEncoder->ViewportSettings->bOffsetRanges,
					AutoEncoder->ViewportSettings->bOffsetRangesInGrid);

				FTransform RangeStartRootTransform;
				QueryDatabase->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1),
					CharacterSequenceIndices[CharacterIdx], CharacterRangeStarts[CharacterIdx]);

				AutoEncoder::Private::AdjustTransformRootTransform(
					OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
					OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
					OriginalPoseState.CurrPoseData->RootData.RootLinearVelocities[CharacterIdx],
					OriginalPoseState.CurrPoseData->RootData.RootAngularVelocities[CharacterIdx],
					RootOffset,
					RangeStartRootTransform,
					AutoEncoder->ViewportSettings->bDisableRootTranslation,
					AutoEncoder->ViewportSettings->bDisableRootRotation);
			}
		}

		// Compute Reconstruction

		if (bAutoEncoderValid)
		{
			AutoEncoder->ToPoseVectors(
				OriginalPoseVector,
				OriginalPoseState.CurrPoseData->ConstView());

			AutoEncoder->ClampPoseVectors(OriginalPoseVector);
			AutoEncoder->NormalizePoseVectors(OriginalPoseVector);

			if (EncoderInferenceNetwork != AutoEncoder->EncoderNetwork ||
				DecoderInferenceNetwork != AutoEncoder->DecoderNetwork)
			{
				EncoderInferenceNetwork = AutoEncoder->EncoderNetwork;
				DecoderInferenceNetwork = AutoEncoder->DecoderNetwork;
				EncoderInference = EncoderInferenceNetwork->GetNetwork()->CreateInferenceObject(MaxCharacterNum);
				DecoderInference = DecoderInferenceNetwork->GetNetwork()->CreateInferenceObject(MaxCharacterNum);
			}

			if (EncodedVector.Num<1>() != EncoderInference->GetOutputSize() ||
				OriginalPoseVector.Num<1>() != EncoderInference->GetInputSize())
			{
				ViewportClient->SetWarningMessage(LOCTEXT("NetworkSizeWarning", "Warning: Network sizes don't match pose size. Is network asset out-of-date?"));
				Learning::Array::Copy(ReconstructedPoseVector, OriginalPoseVector);
			}
			else
			{
				EncoderPerfCounter.Begin();
				EncoderInference->Evaluate(EncodedVector, OriginalPoseVector);
				EncoderPerfCounter.End();

				DecoderPerfCounter.Begin();
				DecoderInference->Evaluate(ReconstructedPoseVector, EncodedVector);
				DecoderPerfCounter.End();

				AutoEncoder->EncoderInferenceTime = FMath::RoundToInt(FPlatformTime::ToMilliseconds64(EncoderPerfCounter.GetAverage()) * 1000.0);
				AutoEncoder->DecoderInferenceTime = FMath::RoundToInt(FPlatformTime::ToMilliseconds64(DecoderPerfCounter.GetAverage()) * 1000.0);
			}

			// Integrate root

			AutoEncoder->DenormalizePoseVectors(ReconstructedPoseVector);
			AutoEncoder->ClampPoseVectors(ReconstructedPoseVector);

			if (AutoEncoder->ViewportSettings->bIntegrateRootOnReconstruction && EditorToolKit->TimelineModel->GetTransportPlaybackMode() == EPlaybackMode::Type::PlayingForward)
			{
				ReconstructedRootLocations = ReconstructedPoseState.CurrPoseData->RootData.RootLocations;
				ReconstructedRootRotations = ReconstructedPoseState.CurrPoseData->RootData.RootRotations;

				AutoEncoder->FromPoseVectors(
					ReconstructedPoseState.CurrPoseData->View(),
					ReconstructedPoseVector,
					ReconstructedRootLocations,
					ReconstructedRootRotations);

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					const float PlaybackDeltaTime = DeltaTime * EditorToolKit->TimelineModel->GetTimelinePlaybackRate();
					ReconstructedPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] = PlaybackDeltaTime * (FVector)ReconstructedPoseState.CurrPoseData->RootData.RootLinearVelocities[CharacterIdx] + ReconstructedRootLocations[CharacterIdx];
					ReconstructedPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx] = FQuat4f::MakeFromRotationVector(PlaybackDeltaTime * ReconstructedPoseState.CurrPoseData->RootData.RootAngularVelocities[CharacterIdx]) * ReconstructedRootRotations[CharacterIdx];
				}
			}
			else
			{
				AutoEncoder->FromPoseVectors(
					ReconstructedPoseState.CurrPoseData->View(),
					ReconstructedPoseVector,
					OriginalPoseState.CurrPoseData->RootData.RootLocations,
					OriginalPoseState.CurrPoseData->RootData.RootRotations);
			}

			// Override with default pose if requested

			if (AutoEncoder->ViewportSettings->bShowDefaultPose)
			{
				AutoEncoder->SetDefaultPoseData(OriginalPoseState.CurrPoseData->View());
				AutoEncoder->SetDefaultPoseData(ReconstructedPoseState.CurrPoseData->View());
			}
		}
		else
		{
			ReconstructedPoseState.PoseIdx = OriginalPoseState.PoseIdx;
			ReconstructedPoseState.BoneParents = OriginalPoseState.BoneParents;
			*ReconstructedPoseState.CurrPoseData = *OriginalPoseState.CurrPoseData;
			*ReconstructedPoseState.PoseGlobalBoneData = *OriginalPoseState.PoseGlobalBoneData;
		}

		// Compute Forward Kinematics

		if (!bAutoEncoderValid)
		{
			AnimDatabase::PoseData::ForwardKinematics(
				OriginalPoseState.PoseGlobalBoneData->View(),
				OriginalPoseState.CurrPoseData->LocalBoneData.ConstView(),
				OriginalPoseState.CurrPoseData->RootData.ConstView(),
				BoneParents);
		}
		else
		{
			AnimDatabase::PoseData::ForwardKinematics(
				OriginalPoseState.PoseGlobalBoneData->View(),
				OriginalPoseState.CurrPoseData->LocalBoneData.ConstView(),
				OriginalPoseState.CurrPoseData->RootData.ConstView(),
				BoneParents);

			AnimDatabase::PoseData::ForwardKinematics(
				ReconstructedPoseState.PoseGlobalBoneData->View(),
				ReconstructedPoseState.CurrPoseData->LocalBoneData.ConstView(),
				ReconstructedPoseState.CurrPoseData->RootData.ConstView(),
				BoneParents);
		}

		// Update Pose

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			if (PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset())
			{
				if (!bAutoEncoderValid)
				{
					const int32 BoneNum = QueryDatabase->GetBoneNum();

					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						const FTransform BoneTransform = FTransform(
							((FQuat)OriginalPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx][BoneIdx]).GetNormalized(),
							(FVector)OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx],
							(FVector)OriginalPoseState.PoseGlobalBoneData->BoneScales[CharacterIdx][BoneIdx]);

						PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetBoneTransformByName(QueryDatabase->GetBoneName(BoneIdx), BoneTransform, EBoneSpaces::WorldSpace);
					}

					PreviewScene->GetPoseableMeshComponent(CharacterIdx)->RefreshBoneTransforms();
				}
				else
				{

					const int32 BoneNum = AutoEncoder->GetBoneNum();

					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						if (AutoEncoder->ViewportSettings->bMeshOnReconstruction)
						{
							const FTransform BoneTransform = FTransform(
								((FQuat)ReconstructedPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx][BoneIdx]).GetNormalized(),
								(FVector)ReconstructedPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx],
								(FVector)ReconstructedPoseState.PoseGlobalBoneData->BoneScales[CharacterIdx][BoneIdx]);

							PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetBoneTransformByName(AutoEncoder->GetBoneName(BoneIdx), BoneTransform, EBoneSpaces::WorldSpace);
						}
						else
						{
							const FTransform BoneTransform = FTransform(
								((FQuat)OriginalPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx][BoneIdx]).GetNormalized(),
								(FVector)OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx],
								(FVector)OriginalPoseState.PoseGlobalBoneData->BoneScales[CharacterIdx][BoneIdx]);

							PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetBoneTransformByName(AutoEncoder->GetBoneName(BoneIdx), BoneTransform, EBoneSpaces::WorldSpace);
						}

						PreviewScene->GetPoseableMeshComponent(CharacterIdx)->RefreshBoneTransforms();
					}
				}
			}
		}

		// Update test range time if playing animation

		EditorToolKit->TimelineModel->UpdateTimelineFrameTime(DeltaTime * QueryDatabase->GetFrameRate().AsDecimal());
	}

	void FAutoEncoderMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		if (TSharedPtr<FAutoEncoderToolkit> ToolKit = WeakToolkit.Pin())
		{
			const int32 CharacterNum = FMath::Min(ActiveRanges.Num(), MaxCharacterNum);

			if (!ToolKit->AutoEncoder->TrainingSettings->Database) { return; }

			// Draw Original Skeleton

			if (ToolKit->AutoEncoder->ViewportSettings->bDrawOriginalSkeleton ||
				ToolKit->AutoEncoder->ViewportSettings->bDrawReconstructedSkeleton)
			{
				// Find bone indices to render

				TArray<int32> BoneIndices;

				if (bAutoEncoderValid && ToolKit->AutoEncoder->ViewportSettings->bDrawRequiredBonesOnly)
				{
					BoneIndices = ToolKit->AutoEncoder->AutoEncodedRequiredBoneIndices;
				}
				else
				{
					const int32 BoneNum = ToolKit->AutoEncoder->TrainingSettings->Database->GetBoneNum();

					BoneIndices.SetNumUninitialized(BoneNum);
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						BoneIndices[BoneIdx] = BoneIdx;
					}
				}

				// Loop over characters and draw them

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					// Draw Original

					if (ToolKit->AutoEncoder->ViewportSettings->bDrawOriginalSkeleton)
					{
						// Draw Skeleton

						FDrawDebugSkeletonSettings SkeletonSettings;
						SkeletonSettings.bDrawSimpleSkeleton = ToolKit->AutoEncoder->ViewportSettings->bDrawSimpleSkeleton;
						SkeletonSettings.BoneRadius = ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale;

						UDrawDebugLibrary::DrawDebugSkeletonArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
							OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
							OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							OriginalPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
							BoneIndices,
							BoneParents,
							ToolKit->AutoEncoder->ViewportSettings->DrawOriginalSkeletonColor,
							false,
							SkeletonSettings);

						// Draw Velocities

						FLinearColor VelocityBoneColor = ToolKit->AutoEncoder->ViewportSettings->DrawOriginalSkeletonColor;
						VelocityBoneColor.A = ToolKit->AutoEncoder->ViewportSettings->DrawLinearVelocitiesOpacity;

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawLinearVelocities)
						{
							UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								OriginalPoseState.PoseGlobalBoneData->BoneLinearVelocities[CharacterIdx],
								BoneIndices,
								VelocityBoneColor,
								ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale * 0.5f,
								false,
								ToolKit->AutoEncoder->ViewportSettings->DrawLinearVelocitiesScale);
						}

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawAngularVelocities)
						{
							UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								OriginalPoseState.PoseGlobalBoneData->BoneAngularVelocities[CharacterIdx],
								BoneIndices,
								VelocityBoneColor,
								ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale * 0.5f,
								false,
								ToolKit->AutoEncoder->ViewportSettings->DrawAngularVelocitiesScale);
						}

						// Draw Transforms

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawBoneTransforms)
						{
							UDrawDebugLibrary::DrawDebugTransformsArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								OriginalPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								OriginalPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
								OriginalPoseState.PoseGlobalBoneData->BoneScales[CharacterIdx],
								UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(ToolKit->AutoEncoder->ViewportSettings->DrawOriginalSkeletonColor),
								false);
						}
					}

					// Draw Reconstructed

					if (bAutoEncoderValid && ToolKit->AutoEncoder->ViewportSettings->bDrawReconstructedSkeleton)
					{
						// Draw Skeleton

						FDrawDebugSkeletonSettings SkeletonSettings;
						SkeletonSettings.bDrawSimpleSkeleton = ToolKit->AutoEncoder->ViewportSettings->bDrawSimpleSkeleton;
						SkeletonSettings.BoneRadius = ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale;

						UDrawDebugLibrary::DrawDebugSkeletonArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							ReconstructedPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
							ReconstructedPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
							ReconstructedPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							ReconstructedPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
							BoneIndices,
							BoneParents,
							ToolKit->AutoEncoder->ViewportSettings->DrawReconstructedSkeletonColor,
							false,
							SkeletonSettings);

						// Draw Velocities

						FLinearColor VelocityBoneColor = ToolKit->AutoEncoder->ViewportSettings->DrawReconstructedSkeletonColor;
						VelocityBoneColor.A = ToolKit->AutoEncoder->ViewportSettings->DrawLinearVelocitiesOpacity;

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawLinearVelocities)
						{
							UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								ReconstructedPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								ReconstructedPoseState.PoseGlobalBoneData->BoneLinearVelocities[CharacterIdx],
								BoneIndices,
								VelocityBoneColor,
								ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale * 0.5f,
								false,
								ToolKit->AutoEncoder->ViewportSettings->DrawLinearVelocitiesScale);
						}

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawAngularVelocities)
						{
							UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								ReconstructedPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								ReconstructedPoseState.PoseGlobalBoneData->BoneAngularVelocities[CharacterIdx],
								BoneIndices,
								VelocityBoneColor,
								ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale * 0.5f,
								false,
								ToolKit->AutoEncoder->ViewportSettings->DrawAngularVelocitiesScale);
						}

						// Draw Transforms

						if (ToolKit->AutoEncoder->ViewportSettings->bDrawBoneTransforms)
						{
							UDrawDebugLibrary::DrawDebugTransformsArrayView(
								FDebugDrawer::MakeDebugDrawer(PDI),
								ReconstructedPoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
								ReconstructedPoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
								ReconstructedPoseState.PoseGlobalBoneData->BoneScales[CharacterIdx],
								UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(ToolKit->AutoEncoder->ViewportSettings->DrawReconstructedSkeletonColor),
								false,
								ToolKit->AutoEncoder->ViewportSettings->DrawRootScale);
						}
					}
				}
			}

			// Draw Range Identifiers

			if (ToolKit->AutoEncoder->ViewportSettings->bDrawRangeIdentifier)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->AutoEncoder->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->AutoEncoder->ViewportSettings->RangeIdentifierColorOverride : ToolKit->AutoEncoder->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->AutoEncoder->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector IdentifierLocation = (FVector)OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
						(FVector)OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector(FVector3f(0.0f, 0.0f, ToolKit->AutoEncoder->ViewportSettings->DrawRangeIdentifierHeight));

					FDrawDebugLineStyle LineStyle;
					LineStyle.Color = IdentifierColor;
					LineStyle.Thickness = ToolKit->AutoEncoder->ViewportSettings->DrawRangeIdentifierThickness;

					UDrawDebugLibrary::DrawDebugSimpleSphere(
						FDebugDrawer::MakeDebugDrawer(PDI),
						IdentifierLocation,
						FRotator::ZeroRotator,
						LineStyle,
						true,
						ToolKit->AutoEncoder->ViewportSettings->DrawRangeIdentifierRadius);
				}
			}

			// Draw Skeleton Roots

			if (ToolKit->AutoEncoder->ViewportSettings->bDrawRoot)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->AutoEncoder->ViewportSettings->DrawRootOpacity;
				LineStyle.Thickness = ToolKit->AutoEncoder->ViewportSettings->DrawSkeletonScale;

				UDrawDebugLibrary::DrawDebugRotationsQuatArrayView(
					FDebugDrawer::MakeDebugDrawer(PDI),
					OriginalPoseState.CurrPoseData->RootData.RootLocations,
					OriginalPoseState.CurrPoseData->RootData.RootRotations,
					LineStyle,
					true,
					ToolKit->AutoEncoder->ViewportSettings->DrawRootScale);
			}

			// Draw Origin

			if (ToolKit->AutoEncoder->ViewportSettings->bDrawOrigin)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->AutoEncoder->ViewportSettings->DrawOriginOpacity;
				LineStyle.Thickness = ToolKit->AutoEncoder->ViewportSettings->DrawOriginLineThickness;

				UDrawDebugLibrary::DrawDebugTransform(
					FDebugDrawer::MakeDebugDrawer(PDI),
					FTransform::Identity,
					LineStyle,
					true,
					ToolKit->AutoEncoder->ViewportSettings->DrawOriginScale);
			}

			// Draw Trajectory

			if (ToolKit->AutoEncoder->ViewportSettings->bDrawTrajectories &&
				(!ToolKit->AutoEncoder->ViewportSettings->bRangesStartAtOrigin ||
					!ToolKit->AutoEncoder->ViewportSettings->bDisableRootTranslation))
			{
				FVector ForwardVector = FVector::ForwardVector;
				switch (ToolKit->AutoEncoder->TrainingSettings->Database->Skeleton->GetPreviewForwardAxis())
				{
				case EAxis::X: ForwardVector = FVector::XAxisVector; break;
				case EAxis::Y: ForwardVector = FVector::YAxisVector; break;
				case EAxis::Z: ForwardVector = FVector::ZAxisVector; break;
				}

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					const FVector RootOffset = AutoEncoder::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->AutoEncoder->ViewportSettings->RangeOffsetSpacing,
						ToolKit->AutoEncoder->ViewportSettings->bOffsetRanges,
						ToolKit->AutoEncoder->ViewportSettings->bOffsetRangesInGrid);

					UDrawDebugLibrary::DrawDebugRangeTrajectoryArrayView(
						FDebugDrawer::MakeDebugDrawer(PDI),
						RangeTrajectoryLocations[ActiveRanges[CharacterIdx]],
						RangeTrajectoryRotations[ActiveRanges[CharacterIdx]],
						true,
						ForwardVector,
						ToolKit->AutoEncoder->ViewportSettings->bDrawTrajectoryOrientations,
						ToolKit->AutoEncoder->ViewportSettings->bRangesStartAtOrigin,
						RootOffset);
				}
			}

			// Draw Encoding Visualization

			if (bAutoEncoderValid && ToolKit->AutoEncoder->ViewportSettings->bDrawEncodingVisualization && PCAEncoder.IsValid())
			{
				const float Opacity = ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationOpacity;
				const float Thickness = ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationThickness;
				const float SpaceScale = ToolKit->AutoEncoder->ViewportSettings->bDrawSharedEncodingVisualization ?
					1.0f * ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationScale :
					0.5f * ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationScale;

				const Learning::FFrameRangeSet& FrameRanges = *ToolKit->AutoEncoder->TrainingSettings->QueryRanges.FrameRangeSet;

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->AutoEncoder->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->AutoEncoder->ViewportSettings->RangeIdentifierColorOverride : ToolKit->AutoEncoder->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = Opacity;

					int32 EntryIdx = INDEX_NONE, RangeIdx = INDEX_NONE;
					FrameRanges.FindTotalRange(EntryIdx, RangeIdx, ActiveRanges[CharacterIdx]);

					FVector SpaceOffset = FVector(0.0f, 0.0f, ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationHeight);

					if (!ToolKit->AutoEncoder->ViewportSettings->bDrawSharedEncodingVisualization)
					{
						SpaceOffset = (FVector)OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
							(FVector)OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector((FVector3f)SpaceOffset);
					}

					const int32 FrameNum = FrameRanges.GetEntryRangeLength(EntryIdx, RangeIdx);

					TArray<TPair<FVector, FVector>> LineSegments;
					LineSegments.SetNumUninitialized(FMath::Max(FrameNum - 1, 0));

					for (int32 FrameIdx = 0; FrameIdx < FrameNum - 1; FrameIdx++)
					{
						const int32 Offset = FrameRanges.GetEntryRangeOffset(EntryIdx, RangeIdx) + FrameIdx;
						const FVector FinalStart = SpaceScale * FVector(PCAEncodedPoses[Offset + 0][0], PCAEncodedPoses[Offset + 0][1], PCAEncodedPoses[Offset + 0][2]) + SpaceOffset;
						const FVector FinalEnd = SpaceScale * FVector(PCAEncodedPoses[Offset + 1][0], PCAEncodedPoses[Offset + 1][1], PCAEncodedPoses[Offset + 1][2]) + SpaceOffset;
						LineSegments[FrameIdx] = { FinalStart, FinalEnd };
					}

					FDrawDebugLineStyle LineStyle;
					LineStyle.Color = IdentifierColor;
					LineStyle.Thickness = Thickness;

					UDrawDebugLibrary::DrawDebugLinesPairsArrayView(FDebugDrawer::MakeDebugDrawer(PDI), LineSegments, LineStyle);
				}

				if (EncodedVector.Num<1>() == PCAEncoder->DimensionNum())
				{
					const int32 EncodedDimNum = EncodedVector.Num<1>();
					
					NormalizedEncodedVector.SetNumUninitialized({ CharacterNum, EncodedDimNum });
					for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
					{
						for (int32 DimIdx = 0; DimIdx < EncodedDimNum; DimIdx++)
						{
							NormalizedEncodedVector[CharacterIdx][DimIdx] = (EncodedVector[CharacterIdx][DimIdx] - PCAEncoderMean[DimIdx]) / FMath::Max(PCAEncoderStd, UE_SMALL_NUMBER);
						}
					}

					TLearningArray<2, float, TInlineAllocator<3>> CurrentEncoding;
					CurrentEncoding.SetNumUninitialized({ CharacterNum, 3 });
					PCAEncoder->Transform(CurrentEncoding, NormalizedEncodedVector);

					for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
					{
						const FLinearColor IdentifierColor = ToolKit->AutoEncoder->ViewportSettings->bOverrideRangeIdentifierColor ?
							ToolKit->AutoEncoder->ViewportSettings->RangeIdentifierColorOverride : ToolKit->AutoEncoder->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;

						FVector SpaceOffset = FVector(0.0f, 0.0f, ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationHeight);

						if (!ToolKit->AutoEncoder->ViewportSettings->bDrawSharedEncodingVisualization)
						{
							SpaceOffset = (FVector)OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
								(FVector)OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector((FVector3f)SpaceOffset);
						}

						const FVector FinalPoint = SpaceScale * FVector(CurrentEncoding[CharacterIdx][0], CurrentEncoding[CharacterIdx][1], CurrentEncoding[CharacterIdx][2]) + SpaceOffset;

						FDrawDebugLineStyle LineStyle;
						LineStyle.Color = IdentifierColor;
						LineStyle.Thickness = Thickness;

						UDrawDebugLibrary::DrawDebugSimpleSphere(
							FDebugDrawer::MakeDebugDrawer(PDI),
							FinalPoint,
							FRotator::ZeroRotator,
							LineStyle,
							true,
							5.0f);
					}
				}

				if (ToolKit->AutoEncoder->ViewportSettings->bDrawSharedEncodingVisualization)
				{
					FVector SpaceOffset = FVector(0.0f, 0.0f, ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationHeight);

					FDrawDebugLineStyle LineStyle;
					LineStyle.Color = FLinearColor::Black;
					LineStyle.Thickness = 1.0f;

					UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(SpaceScale, 0.0, 0.0), LineStyle);
					UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(0.0, SpaceScale, 0.0), LineStyle);
					UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(0.0, 0.0, SpaceScale), LineStyle);
				}
				else
				{
					for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
					{
						FVector SpaceOffset = FVector(0.0f, 0.0f, ToolKit->AutoEncoder->ViewportSettings->DrawEncodingVisualizationHeight);

						if (!ToolKit->AutoEncoder->ViewportSettings->bDrawSharedEncodingVisualization)
						{
							SpaceOffset = (FVector)OriginalPoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
								(FVector)OriginalPoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector((FVector3f)SpaceOffset);
						}

						FDrawDebugLineStyle LineStyle;
						LineStyle.Color = FLinearColor::Black;
						LineStyle.Thickness = 1.0f;

						UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(SpaceScale, 0.0, 0.0), LineStyle);
						UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(0.0, SpaceScale, 0.0), LineStyle);
						UDrawDebugLibrary::DrawDebugLine(FDebugDrawer::MakeDebugDrawer(PDI), SpaceOffset, SpaceOffset + 10.0 * FVector(0.0, 0.0, SpaceScale), LineStyle);
					}
				}
			}

			// Debug Draw

			if (bAutoEncoderValid && (ToolKit->AutoEncoder->TrainingSettings->DatabaseDebugDrawer || ToolKit->AutoEncoder->TrainingSettings->AutoEncoderDebugDrawer))
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->AutoEncoder->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->AutoEncoder->ViewportSettings->RangeIdentifierColorOverride : ToolKit->AutoEncoder->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->AutoEncoder->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector RootOffset = AutoEncoder::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->AutoEncoder->ViewportSettings->RangeOffsetSpacing,
						ToolKit->AutoEncoder->ViewportSettings->bOffsetRanges,
						ToolKit->AutoEncoder->ViewportSettings->bOffsetRangesInGrid);

					FTransform RangeStartRootTransform = FTransform::Identity;
					ToolKit->AutoEncoder->TrainingSettings->Database->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1),
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeStarts[CharacterIdx]);

					const FTransform RangeViewportTransform = !ToolKit->AutoEncoder->ViewportSettings->bRangesStartAtOrigin ? FTransform::Identity
						: RangeStartRootTransform.Inverse() * FTransform(FQuat::Identity, RootOffset, FVector::OneVector);

					OriginalPoseState.PoseIdx = CharacterIdx;
					ReconstructedPoseState.PoseIdx = CharacterIdx;

					if (ToolKit->AutoEncoder->TrainingSettings->DatabaseDebugDrawer)
					{
						ToolKit->AutoEncoder->TrainingSettings->DatabaseDebugDrawer->DrawDebug(
							FDebugDrawer::MakeDebugDrawer(PDI),
							FDebugDrawer::MakeDebugDrawer(static_cast<AnimDatabase::Editor::FViewportClient*>(Viewport->GetClient())->GetOrMakeDebugDrawBuffer()),
							OriginalPoseState,
							ToolKit->AutoEncoder->TrainingSettings->Database.Get(),
							ToolKit->AutoEncoder->TrainingSettings->QueryRanges,
							RangeViewportTransform,
							CharacterIdx,
							CharacterSequenceIndices[CharacterIdx],
							CharacterRangeTimes[CharacterIdx],
							CharacterRangeStarts[CharacterIdx],
							CharacterRangeLengths[CharacterIdx],
							IdentifierColor);
					}

					if (ToolKit->AutoEncoder->TrainingSettings->AutoEncoderDebugDrawer)
					{
						ToolKit->AutoEncoder->TrainingSettings->AutoEncoderDebugDrawer->DrawDebug(
							FDebugDrawer::MakeDebugDrawer(PDI),
							FDebugDrawer::MakeDebugDrawer(static_cast<AnimDatabase::Editor::FViewportClient*>(Viewport->GetClient())->GetOrMakeDebugDrawBuffer()),
							OriginalPoseState,
							ReconstructedPoseState,
							ToolKit->AutoEncoder->TrainingSettings->Database.Get(),
							ToolKit->AutoEncoder->TrainingSettings->QueryRanges,
							RangeViewportTransform,
							ToolKit->AutoEncoder,
							CharacterIdx,
							CharacterSequenceIndices[CharacterIdx],
							CharacterRangeTimes[CharacterIdx],
							CharacterRangeStarts[CharacterIdx],
							CharacterRangeLengths[CharacterIdx],
							IdentifierColor,
							ToolKit->AutoEncoder->ViewportSettings->DrawOriginalSkeletonColor,
							ToolKit->AutoEncoder->ViewportSettings->DrawReconstructedSkeletonColor);
					}
				}
			}
		}

		FEdMode::Render(View, Viewport, PDI);
	}
}

#undef LOCTEXT_NAMESPACE