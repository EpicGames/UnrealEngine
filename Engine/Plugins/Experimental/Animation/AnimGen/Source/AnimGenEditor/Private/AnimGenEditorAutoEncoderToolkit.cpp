// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditorAutoEncoderToolkit.h"

#include "AnimGenEditorAutoEncoderMode.h"
#include "AnimGenEditorTraining.h"
#include "SAnimGenEditorTraining.h"

#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorTimeline.h"
#include "AnimDatabaseEditorViewportClient.h"
#include "SAnimDatabaseEditorViewport.h"
#include "SAnimDatabaseEditorTimeline.h"

#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Viewports.h"
#include "PreviewProfileController.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SScrollBox.h"

#include "AnimGenLog.h"
#include "AnimGenAutoEncoder.h"

#include "AnimDatabase.h"
#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameAttribute.h"
#include "AnimDatabasePose.h"

#include "LearningArray.h"
#include "LearningSharedMemory.h"
#include "LearningTrainer.h"
#include "LearningNeuralNetwork.h"
#include "LearningRandom.h"
#include "LearningProgress.h"

#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "AdvancedPreviewSceneModule.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AssetTypeCategories.h"
#include "PipInstallHelpers.h"

#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"

#include "Misc/Guid.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "Async/ParallelFor.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "NNERuntimeBasicCpuBuilder.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorAutoEncoderToolkit"


FText UAnimGenEditorAutoEncoderAssetDefinition::GetAssetDisplayName() const
{
	return LOCTEXT("AnimGenEditorAutoEncoderAssetDefinitionName", "Auto-Encoder");
}

FLinearColor UAnimGenEditorAutoEncoderAssetDefinition::GetAssetColor() const
{
	return FColor(30, 104, 122);
}

TSoftClassPtr<UObject> UAnimGenEditorAutoEncoderAssetDefinition::GetAssetClass() const
{
	return UAnimGenAutoEncoder::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAnimGenEditorAutoEncoderAssetDefinition::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("AnimGenAutoEncoderAssetDefinitionMenu", "AnimGen")) };
	return Categories;
}

EAssetCommandResult UAnimGenEditorAutoEncoderAssetDefinition::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FPythonScriptInitHelper::InitPythonAndPipInstall(FSimpleDelegate::CreateLambda([
		OpenArgs = FAssetOpenArgs(OpenArgs),
		// Taking a copy of OpenArgs.Assets as it's a TArrayView and this callback outlives the view
		OpenAssetsData = TArray<FAssetData>(OpenArgs.Assets)]()
		{
			FAssetArgs WrappedArgs(OpenAssetsData);
			for (UAnimGenAutoEncoder* Asset : WrappedArgs.LoadObjects<UAnimGenAutoEncoder>())
			{
				TSharedRef<UE::AnimGen::Editor::FAutoEncoderToolkit> NewEditor(new UE::AnimGen::Editor::FAutoEncoderToolkit());
				NewEditor->InitAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Asset);
			}
		}),
		FSimpleDelegate::CreateLambda([]()
			{
				UE_LOGF(LogAnimGen, Warning, "AnimGen AutoEncoder toolkit may not function properly without Python enabled");
			}));

	return EAssetCommandResult::Handled;
}

UAnimGenAutoEncoderEditorFactory::UAnimGenAutoEncoderEditorFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAnimGenAutoEncoder::StaticClass();
}

UObject* UAnimGenAutoEncoderEditorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimGenAutoEncoder>(InParent, Name, Flags | RF_Transactional);
}

bool UAnimGenAutoEncoderEditorFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UAnimGenAutoEncoderEditorFactory::ConfigureProperties()
{
	return true;
}

FText UAnimGenAutoEncoderEditorFactory::GetDisplayName() const
{
	return LOCTEXT("AnimGenAutoEncoderEditorAsset_DisplayName", "Auto-Encoder");
}

uint32 UAnimGenAutoEncoderEditorFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UAnimGenAutoEncoderEditorFactory::GetToolTip() const
{
	return LOCTEXT("AnimGenAutoEncoderEditorAsset_Tooltip", "Asset for training a model which creates a compressed encoding of animation data suitable for machine learning tasks.");
}

FString UAnimGenAutoEncoderEditorFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("AGAE_NewAnimGenAutoEncoder"));
}

const TArray<FText>& UAnimGenAutoEncoderEditorFactory::GetMenuCategorySubMenus() const
{
	static TArray<FText> SubMenus{ LOCTEXT("SubMenuAnimGen", "AnimGen") };
	return SubMenus;
}

namespace UE::AnimGen::Editor
{
	TSharedRef<IDetailCustomization> FAutoEncoderTrainingSettingsDetails::MakeInstance()
	{
		return MakeShared<FAutoEncoderTrainingSettingsDetails>();
	}

	void FAutoEncoderTrainingSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Get the UAnimGenAutoEnco object we are associated with

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1) { return; }

		UAnimGenAutoEncoderTrainingSettings* TrainingSettings = StaticCast<UAnimGenAutoEncoderTrainingSettings*>(Objects[0].Pin().Get());
		if (!TrainingSettings) { return; }

		// Reset the selected items since we are re-drawing the UI

		SelectedItems.Reset();

		// Put the Data category first

		IDetailCategoryBuilder& DataCategory = DetailBuilder.EditCategory("Data", LOCTEXT("DataCategory", "Data"), ECategoryPriority::Important);
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, Database), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, FrameRanges), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, ExcludedBones), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, bExcludeVirtualBones), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, RootWeightMultiplier), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, AttributeWeightMultiplier), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, BaseWeightMultiplier), UAnimGenAutoEncoderTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoderTrainingSettings, FrameAttributes), UAnimGenAutoEncoderTrainingSettings::StaticClass());

		// Put the frame ranges returned by the query next

		IDetailCategoryBuilder& RangesCategory = DetailBuilder.EditCategory("Ranges", LOCTEXT("RangesCategory", "Ranges"), ECategoryPriority::Important);

		// Add the ListView 

		if (TrainingSettings->QueryEntries.Num() > 0)
		{
			// Make the ListView used to render the FQueryEntry array

			SAssignNew(ListView, SListView<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>)
				.ListItemsSource(&TrainingSettings->QueryEntries)
				.OnGenerateRow(this, &FAutoEncoderTrainingSettingsDetails::OnGenerateRow)
				.OnSelectionChanged(this, &FAutoEncoderTrainingSettingsDetails::OnSelectionChanged)
				.HeaderRow(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Icon")
					.DefaultLabel(FText::GetEmpty())
					.FillWidth(0.05f)
					+ SHeaderRow::Column("Sequence")
					.DefaultLabel(LOCTEXT("ColumnLabelAnimSequence", "Anim Sequence"))
					.FillWidth(0.7f)
					+ SHeaderRow::Column("Start")
					.DefaultLabel(LOCTEXT("ColumnLabelStart", "Start"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Stop")
					.DefaultLabel(LOCTEXT("ColumnLabelStop", "Stop"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Mirrored")
					.DefaultLabel(LOCTEXT("ColumnLabelMirrored", "Mirrored"))
					.FillWidth(0.05f)
				);

			// Select first entry if we are re-drawing the UI

			ListView->SetSelection(TrainingSettings->QueryEntries[0]);

			// Add to UI

			RangesCategory.AddCustomRow(LOCTEXT("RangesProperty", "Ranges"))
				.WholeRowContent()
				[
					SNew(SBox)
						// We need to set the list view to a fixed maximum height otherwise it will fill the whole details panel. Unfortunately the 
						// scroll bar of the details panel will get messed up if we make this too large and this is a wont-fix right now (UE-40387)
						// we therefore try to limit the size a bit even if this is annoying for users.
						.MaxDesiredHeight(500.0f)
						[
							ListView.ToSharedRef()
						]
				];

			// Re-set selection

			TArray<TSharedPtr<AnimDatabase::Editor::FQueryEntry>, TInlineAllocator<16>> ResetSelectedItems;
			for (const TSharedPtr<AnimDatabase::Editor::FQueryEntry>& Item : ListView->GetItems())
			{
				if (Item->bIsSelected)
				{
					ResetSelectedItems.Add(Item);
				}
			}
			ListView->SetItemSelection(ResetSelectedItems, true);
		}
		else
		{
			// Write out message instead of query returned no ranges

			RangesCategory.AddCustomRow(LOCTEXT("RangesProperty", "Ranges"))
				.WholeRowContent()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(
						TrainingSettings->Database.IsNull() ? LOCTEXT("NoDatabase", "No Training Database Provided") :
						!TrainingSettings->Database.Get() ? LOCTEXT("LoadingDatabase", "Loading Training Database...") :
						LOCTEXT("EmptyRanges", "No ranges returned by Query"))
				];
		}
	}

	void FAutoEncoderTrainingSettingsDetails::OnSelectionChanged(TSharedPtr<AnimDatabase::Editor::FQueryEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		ListView->GetSelectedItems(SelectedItems);
		for (const TSharedPtr<AnimDatabase::Editor::FQueryEntry>& Item : ListView->GetItems())
		{
			Item->bIsSelected = SelectedItems.Contains(Item);
		}
	}

	TSharedRef<ITableRow> FAutoEncoderTrainingSettingsDetails::OnGenerateRow(TSharedPtr<AnimDatabase::Editor::FQueryEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		class SFilterMultiColumnRow : public SMultiColumnTableRow<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>
		{
		public:
			SLATE_BEGIN_ARGS(SFilterMultiColumnRow) {}
				SLATE_ARGUMENT(TSharedPtr<AnimDatabase::Editor::FQueryEntry>, Item)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
			{
				Item = InArgs._Item;
				SMultiColumnTableRow<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
			}

			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
			{
				if (ColumnName == "Icon")
				{
					// Draw colored circle icon a.k.a "Range Identifier"
					return SNew(SBox)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.MaxDesiredWidth(10.0f)
						.MaxDesiredHeight(10.0f)
						.MaxAspectRatio(1.0f)
						.MinAspectRatio(1.0f)
						[
							SNew(SImage)
								.ColorAndOpacity(Item->Color)
								.Image(FCoreStyle::Get().GetBrush("GraphEditor.PinIcon"))
						];
				}
				else if (ColumnName == "Sequence")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							Item->Sequence ?
							SNew(SHyperlink).Text(FText::FromString(Item->Sequence->GetName())).OnNavigate_Lambda([this]()
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Item->Sequence);
								}) :
							SNew(SHyperlink).Text(LOCTEXT("NullSequenceName", "null"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(Item->bIsMirrored ? LOCTEXT("MirroredAddition", " (mirrored)") : FText::GetEmpty())
						];
				}
				else if (ColumnName == "Start")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StartFrame))
						];
				}
				else if (ColumnName == "Stop")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StopFrame))
						];
				}
				else if (ColumnName == "Mirrored")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SImage).Image(FCoreStyle::Get().GetBrush("GraphEditor.AlignNodesCenter")).RenderOpacity(Item->bIsMirrored ? 1.0f : 0.0f)
						];
				}

				return SNullWidget::NullWidget;
			}

		private:
			TSharedPtr<AnimDatabase::Editor::FQueryEntry> Item;
		};

		return SNew(SFilterMultiColumnRow, OwnerTable).Item(Item);
	}

	TSharedRef<IDetailCustomization> FAutoEncoderDetails::MakeInstance()
	{
		return MakeShared<FAutoEncoderDetails>();
	}

	void FAutoEncoderDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Get the UAnimGenAutoEncoder object we are associated with

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1) { return; }

		UAnimGenAutoEncoder* AutoEncoder = StaticCast<UAnimGenAutoEncoder*>(Objects[0].Pin().Get());
		if (!AutoEncoder) { return; }

		// Add Categories

		IDetailCategoryBuilder& AutoEncoderCategory = DetailBuilder.EditCategory("AutoEncoder", LOCTEXT("AutoEncoder", "Auto Encoder"), ECategoryPriority::Important);
		AutoEncoderCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, TrainedBoneLocations), UAnimGenAutoEncoder::StaticClass());
		AutoEncoderCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, TrainedBoneRotations), UAnimGenAutoEncoder::StaticClass());
		AutoEncoderCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, TrainedBoneScales), UAnimGenAutoEncoder::StaticClass());
		AutoEncoderCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, TrainedFrameAttributes), UAnimGenAutoEncoder::StaticClass());

		// Check if Trained Database has been modified

		if (AutoEncoder->IsValid() && AutoEncoder->TrainingSettings->Database.Get() && 
			UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(
				AutoEncoder->TrainingSettings->Database.Get(),
				AutoEncoder->TrainingSettings->QueryRanges) != AutoEncoder->TrainedContentHash)
		{
			FDetailWidgetRow& NeedsReimportErrorRow = AutoEncoderCategory.AddCustomRow(LOCTEXT("ModifiedWarning", "Warning"))
				.WholeRowContent()
				[
					SNew(SBox)
						.Padding(FMargin(0.0f, 4.0f))
						[
							SNew(SWarningOrErrorBox)
								.MessageStyle(EMessageStyle::Warning)
								.Message(LOCTEXT("ModifiedWarningMessage", 
									"Training Data appears to have changed since AutoEncoder was last trained."))
						]
				];
		}

		IDetailCategoryBuilder& StatisticsCategory = DetailBuilder.EditCategory("Statistics", LOCTEXT("Statistics", "Statistics"), ECategoryPriority::Important);
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, EncoderSize), UAnimGenAutoEncoder::StaticClass());
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, DecoderSize), UAnimGenAutoEncoder::StaticClass());
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, EncoderInferenceTime), UAnimGenAutoEncoder::StaticClass());
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenAutoEncoder, DecoderInferenceTime), UAnimGenAutoEncoder::StaticClass());
	}

	class SAutoEncoderViewportToolBar;

	/** Custom AutoEncoder Viewport class. Contains a custom show menu at the top which contains some toggles for the ViewportSettings */
	class SAutoEncoderViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
	{
	public:

		SLATE_BEGIN_ARGS(SAutoEncoderViewport) {}
		SLATE_END_ARGS();

		void Construct(
			const FArguments& InArgs,
			const TSharedRef<FAutoEncoderToolkit>& InAssetEditorToolkit,
			const TSharedRef<AnimDatabase::Editor::FPreviewScene>& InPreviewScene,
			const FEditorModeID InModeID);

		// ~ICommonEditorViewportToolbarInfoProvider interface
		virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
		virtual TSharedPtr<FExtender> GetExtenders() const override;
		virtual void OnFloatingButtonClicked() override {}
		// ~End of ICommonEditorViewportToolbarInfoProvider interface

	protected:

		// ~SEditorViewport interface
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
		// ~End of SEditorViewport interface

		/** Viewport client */
		TSharedPtr<AnimDatabase::Editor::FViewportClient> ViewportClient;

		/** The preview scene that we are viewing */
		TWeakPtr<AnimDatabase::Editor::FPreviewScene> PreviewScenePtr;

		/** AutoEncoder Toolkit */
		TWeakPtr<FAutoEncoderToolkit> AutoEncoderToolkit;

		/** Editor Mode */
		FEditorModeID ModeID;
	};

	/** Custom viewport toolbar which refers to some settings in database ViewportSettings */
	class SAutoEncoderViewportToolBar : public SCommonEditorViewportToolbarBase
	{
	public:
		SLATE_BEGIN_ARGS(SAutoEncoderViewportToolBar) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<SAutoEncoderViewport> InViewport, TWeakPtr<FAutoEncoderToolkit> InAutoEncoderToolkit);
		virtual TSharedRef<SWidget> GenerateShowMenu() const override;

		TWeakPtr<FAutoEncoderToolkit> AutoEncoderToolkit;
	};

	void SAutoEncoderViewport::Construct(
		const FArguments& InArgs,
		const TSharedRef<FAutoEncoderToolkit>& InAssetEditorToolkit,
		const TSharedRef<AnimDatabase::Editor::FPreviewScene>& InPreviewScene,
		const FEditorModeID InModeID)
	{
		PreviewScenePtr = InPreviewScene;
		AutoEncoderToolkit = InAssetEditorToolkit.ToWeakPtr();
		ModeID = InModeID;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
		);
	}

	TSharedRef<SEditorViewport> SAutoEncoderViewport::GetViewportWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<FExtender> SAutoEncoderViewport::GetExtenders() const
	{
		TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
		return Result;
	}

	TSharedRef<FEditorViewportClient> SAutoEncoderViewport::MakeEditorViewportClient()
	{
		ViewportClient = MakeShared<AnimDatabase::Editor::FViewportClient>(
			PreviewScenePtr,
			SharedThis(this),
			StaticCastWeakPtr<FAssetEditorToolkit>(AutoEncoderToolkit),
			ModeID);

		ViewportClient->ViewportType = LVT_Perspective;
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
		ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

		return ViewportClient.ToSharedRef();
	}

	TSharedPtr<SWidget> SAutoEncoderViewport::BuildViewportToolbar()
	{
		return SNew(SAutoEncoderViewportToolBar, SharedThis(this), AutoEncoderToolkit);
	}

	void SAutoEncoderViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<SAutoEncoderViewport> InViewport, TWeakPtr<FAutoEncoderToolkit> InAutoEncoderToolkit)
	{
		SCommonEditorViewportToolbarBase::Construct(
			SCommonEditorViewportToolbarBase::FArguments()
			.AddRealtimeButton(false)
			.PreviewProfileController(MakeShared<FPreviewProfileController>()),
			InViewport);

		AutoEncoderToolkit = InAutoEncoderToolkit;
	}

	TSharedRef<SWidget> SAutoEncoderViewportToolBar::GenerateShowMenu() const
	{
		GetInfoProvider().OnFloatingButtonClicked();

		TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
		{
			//-------------------------------

			ShowMenuBuilder.BeginSection("Skeleton", LOCTEXT("ShowMenu_SkeletonLabel", "Skeleton"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_OriginalSkeletonLabelSkeleton", "Original Skeleton"),
				LOCTEXT("ShowMenu_OriginalSkeletonLabelSkeletonToolTip", "Draw the original character skeleton"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOriginalSkeleton = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOriginalSkeleton;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOriginalSkeleton : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_ReconstructedSkeletonLabelSkeleton", "Reconstructed Skeleton"),
				LOCTEXT("ShowMenu_ReconstructedSkeletonLabelSkeletonToolTip", "Draw the reconstructed character skeleton"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawReconstructedSkeleton = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawReconstructedSkeleton;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawReconstructedSkeleton : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocities", "Linear Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocitiesToolTip", "Draw linear velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawLinearVelocities = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawLinearVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawLinearVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocities", "Angular Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocitiesToolTip", "Draw Angular velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawAngularVelocities = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawAngularVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawAngularVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Root", LOCTEXT("ShowMenu_RootLabel", "Root"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RootLabelRoot", "Root"),
				LOCTEXT("ShowMenu_RootLabelRootToolTip", "Draw Root"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRoot = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRoot;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRoot : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Trajectories", LOCTEXT("ShowMenu_TrajectoriesLabel", "Trajectories"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectories", "Trajectories"),
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesToolTip", "Draw Trajectories"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectories = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectories;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectories : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesOrientations", "Trajectories Orientations"),
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesOrientationsToolTip", "Draw Trajectories Orientations"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectoryOrientations = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectoryOrientations;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawTrajectoryOrientations : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Ranges", LOCTEXT("ShowMenu_RangesLabel", "Ranges"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RangesLabelRangeIdentifiers", "Range Identifiers"),
				LOCTEXT("ShowMenu_RangesLabelRangeIdentifiersToolTip", "Draw Range Identifiers"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRangeIdentifier = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRangeIdentifier;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawRangeIdentifier : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Origin", LOCTEXT("ShowMenu_OriginLabel", "Origin"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_OriginLabelOrigin", "Origin"),
				LOCTEXT("ShowMenu_OriginLabelOriginToolTip", "Draw Origin"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (AutoEncoderToolkit.IsValid()) {
				AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOrigin = !AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOrigin;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return AutoEncoderToolkit.IsValid() ? AutoEncoderToolkit.Pin()->AutoEncoder->ViewportSettings->bDrawOrigin : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------
		}

		return ShowMenuBuilder.MakeWidget();
	}

	struct FAutoEncoderTrainingModel : public ITrainingModel
	{
		FAutoEncoderTrainingModel() = default;
		FAutoEncoderTrainingModel(TWeakPtr<FAutoEncoderToolkit> InToolkit) : Toolkit(InToolkit) {}
		virtual ~FAutoEncoderTrainingModel() { if (IsTraining()) { StopTraining(); } }

		virtual bool IsEmpty() const override
		{
			if (TSharedPtr<FAutoEncoderToolkit> ToolkitPtr = Toolkit.Pin())
			{
				return !ToolkitPtr->AutoEncoder->IsValid();
			}

			return false;
		}

		virtual void StartTraining() override
		{
			if (IsTraining()) { return; }

			if (TSharedPtr<FAutoEncoderToolkit> ToolkitPtr = Toolkit.Pin())
			{
				// Clear Error Message

				ErrorMessage = FText();

				// Get Auto-Encoder Asset

				UAnimGenAutoEncoder* AutoEncoder = ToolkitPtr->AutoEncoder;

				// Find Training Database and validate that it is correct

				const UAnimDatabase* TrainingDatabase = AutoEncoder->TrainingSettings->Database.LoadSynchronous();
				if (!TrainingDatabase)
				{
					ErrorMessage = LOCTEXT("DatabaseNotProvidedError", "Error: Database not Provided");
					return;
				}

				const USkeleton* TrainingSkeleton = TrainingDatabase->GetSkeleton();
				if (!TrainingSkeleton)
				{
					ErrorMessage = LOCTEXT("SkeletonNotProvidedError", "Error: Database contains no Skeleton");
					return;
				}

				const int32 BoneNum = TrainingDatabase->GetBoneNum();
				if (BoneNum == 0)
				{
					ErrorMessage = LOCTEXT("NoBonesError", "Error: Skeleton contains no bones");
					return;
				}

				// Compute Training Frame Ranges

				const FAnimDatabaseFrameRanges TrainingFrameRanges = AutoEncoder->TrainingSettings->FrameRanges ?
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(TrainingDatabase, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase), AutoEncoder->TrainingSettings->FrameRanges) :
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase);

				const int32 TotalRangeNum = TrainingFrameRanges.FrameRangeSet->GetTotalRangeNum();

				if (TotalRangeNum == 0)
				{
					ErrorMessage = LOCTEXT("TrainingFrameEmptyError", "Error: Training Frame Ranges are empty");
					return;
				}

				// Checking Empty and Additives

				TArray<int32> RangeSequenceIndices;
				RangeSequenceIndices.SetNumUninitialized(TotalRangeNum);
				Learning::FrameRangeSet::AllRangeSequences(RangeSequenceIndices, *TrainingFrameRanges.FrameRangeSet);

				for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
				{
					const UAnimSequence* AnimSequence = TrainingDatabase->GetAnimSequence(RangeSequenceIndices[RangeIdx]);

					if (!AnimSequence)
					{
						ErrorMessage = LOCTEXT("NullAnimationError", "Error: Training data contains null animations.");
						return;
					}

					if (AnimSequence->IsValidAdditive())
					{
						ErrorMessage = FText::Format(LOCTEXT("AdditiveAnimationError", "Error: Training data contains additive animations: {0}."), FText::FromString(AnimSequence->GetName()));
						return;
					}
				}

				// Allocate and Fill Range Data

				RangeStarts = Learning::SharedMemory::Allocate<1, int32>({ TotalRangeNum });
				RangeLengths = Learning::SharedMemory::Allocate<1, int32>({ TotalRangeNum });

				int32 TotalFrameNum = 0;
				for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
				{
					RangeStarts.View[RangeIdx] = TotalFrameNum;
					RangeLengths.View[RangeIdx] = TrainingFrameRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx];
					TotalFrameNum += TrainingFrameRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx];
				}

				// Create Attributes or Check Attributes Match

				USkeleton* Skeleton = TrainingDatabase->GetSkeleton();
				AutoEncoder->TrainedSkeleton = Skeleton;

				TArray<FAnimDatabaseFrameAttribute> TrainingDatabaseFrameAttributes;
				int32 AttributeNum = AutoEncoder->TrainingSettings->FrameAttributes.Num();

				TrainingDatabaseFrameAttributes.SetNum(AttributeNum);
				AutoEncoder->AttributeNames.SetNumUninitialized(AttributeNum);
				AutoEncoder->AttributeTypes.SetNumUninitialized(AttributeNum);
				AutoEncoder->TrainedFrameAttributes.SetNum(AttributeNum);
				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					AutoEncoder->TrainedFrameAttributes[AttributeIdx].Name = AutoEncoder->TrainingSettings->FrameAttributes[AttributeIdx].Name;
					AutoEncoder->TrainedFrameAttributes[AttributeIdx].FrameAttribute = DuplicateObject(AutoEncoder->TrainingSettings->FrameAttributes[AttributeIdx].FrameAttribute, AutoEncoder);

					TrainingDatabaseFrameAttributes[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(
						TrainingDatabase,
						TrainingFrameRanges,
						AutoEncoder->TrainedFrameAttributes[AttributeIdx].FrameAttribute);

					if (!TrainingDatabaseFrameAttributes[AttributeIdx].IsValid())
					{
						ErrorMessage = FText::Format(LOCTEXT("InvalidFrameAttribute", "Error: Frame Attribute for {0} is invalid."), FText::FromName(AutoEncoder->TrainedFrameAttributes[AttributeIdx].Name));
						AutoEncoder->AttributeNames.Empty();
						AutoEncoder->AttributeTypes.Empty();
						AutoEncoder->TrainedFrameAttributes.Empty();
						return;
					}

					AutoEncoder->AttributeNames[AttributeIdx] = AutoEncoder->TrainingSettings->FrameAttributes[AttributeIdx].Name;
					AutoEncoder->AttributeTypes[AttributeIdx] = TrainingDatabaseFrameAttributes[AttributeIdx].Type;
				}

				AutoEncoder->TrainedContentHash = UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(TrainingDatabase, TrainingFrameRanges);

				// Extract Animation Data

				TrainingDatabase->WaitForCompressionOnAnimSequencesFromArrayView(RangeSequenceIndices);

				AnimDatabase::FPoseData PoseData;
				PoseData.Resize(TotalFrameNum, BoneNum, AutoEncoder->GetAttributeTypes(), AutoEncoder->GetAttributeNames());

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("SamplingAnimationData", "Sampling Animation Data"));
					SlowTask.MakeDialog();

					ParallelFor(TotalRangeNum, [this, TrainingDatabase, &TrainingFrameRanges, &TrainingDatabaseFrameAttributes, &RangeSequenceIndices, &PoseData](int32 RangeIdx)
						{
							TrainingDatabase->GetPoseData(
								PoseData.Slice(RangeStarts.View[RangeIdx], RangeLengths.View[RangeIdx]),
								RangeSequenceIndices[RangeIdx],
								TrainingFrameRanges.FrameRangeSet->GetAllRangeStarts()[RangeIdx],
								TrainingDatabaseFrameAttributes);
						});
				}

				const int32 TotalAttributeSize = PoseData.AttributeData.AttributeData.Num<1>();

				// Computing Averages

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingAverages", "Computing Bone Averages"));
					SlowTask.MakeDialog();

					// Compute Excluded Bones

					const int32 ExcludedBoneNum = AutoEncoder->TrainingSettings->ExcludedBones.Num();
					TArray<int32, TInlineAllocator<64>> ExcludedBones;
					ExcludedBones.Reserve(ExcludedBoneNum);
					for (int32 ExcludedBoneIdx = 0; ExcludedBoneIdx < ExcludedBoneNum; ExcludedBoneIdx++)
					{
						const int32 BoneIndex = TrainingDatabase->FindBoneIndex(AutoEncoder->TrainingSettings->ExcludedBones[ExcludedBoneIdx].BoneName);
						if (BoneIndex != INDEX_NONE)
						{
							ExcludedBones.Add(BoneIndex);
						}
					}

					// Exclude Virtual Bones

					if (AutoEncoder->TrainingSettings->bExcludeVirtualBones)
					{
						for (const FBoneIndexType VirtualBone : Skeleton->GetReferenceSkeleton().GetRequiredVirtualBones())
						{
							ExcludedBones.Add((int32)VirtualBone);
						}
					}

					AutoEncoder->BoneNames.SetNumUninitialized(BoneNum);
					AutoEncoder->BoneParents.SetNumUninitialized(BoneNum);
					AutoEncoder->DefaultBoneLocations.SetNumUninitialized(BoneNum);
					AutoEncoder->DefaultBoneRotations.SetNumUninitialized(BoneNum);
					AutoEncoder->DefaultBoneScales.SetNumUninitialized(BoneNum);

					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						AutoEncoder->BoneNames[BoneIdx] = Skeleton->GetReferenceSkeleton().GetBoneName(BoneIdx);
						AutoEncoder->BoneParents[BoneIdx] = Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIdx);
					}

					AnimDatabase::PoseData::ComputeDefaultBoneValuesAndIndices(
						AutoEncoder->DefaultBoneLocations,
						AutoEncoder->DefaultBoneRotations,
						AutoEncoder->DefaultBoneScales,
						AutoEncoder->AutoEncodedBoneLocationIndices,
						AutoEncoder->AutoEncodedBoneRotationIndices,
						AutoEncoder->AutoEncodedBoneScaleIndices,
						PoseData.LocalBoneData.ConstView(),
						TrainingDatabase->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose(),
						ExcludedBones);

					AnimDatabase::PoseData::ComputeRequiredBoneIndices(
						AutoEncoder->AutoEncodedRequiredBoneIndices,
						AutoEncoder->AutoEncodedBoneLocationIndices,
						AutoEncoder->AutoEncodedBoneRotationIndices,
						AutoEncoder->AutoEncodedBoneScaleIndices);
				}

				// Update Trained Bone Names

				const int32 BoneLocationNum = AutoEncoder->AutoEncodedBoneLocationIndices.Num();
				const int32 BoneRotationNum = AutoEncoder->AutoEncodedBoneRotationIndices.Num();
				const int32 BoneScaleNum = AutoEncoder->AutoEncodedBoneScaleIndices.Num();

				AutoEncoder->TrainedBoneLocations.SetNumUninitialized(BoneLocationNum);
				AutoEncoder->TrainedBoneRotations.SetNumUninitialized(BoneRotationNum);
				AutoEncoder->TrainedBoneScales.SetNumUninitialized(BoneScaleNum);

				for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
				{
					AutoEncoder->TrainedBoneLocations[BoneIdx] = TrainingSkeleton->GetReferenceSkeleton().GetBoneName(AutoEncoder->AutoEncodedBoneLocationIndices[BoneIdx]);
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
				{
					AutoEncoder->TrainedBoneRotations[BoneIdx] = TrainingSkeleton->GetReferenceSkeleton().GetBoneName(AutoEncoder->AutoEncodedBoneRotationIndices[BoneIdx]);
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneScaleNum; BoneIdx++)
				{
					AutoEncoder->TrainedBoneScales[BoneIdx] = TrainingSkeleton->GetReferenceSkeleton().GetBoneName(AutoEncoder->AutoEncodedBoneScaleIndices[BoneIdx]);
				}

				// Compute Pose Vectors

				const int32 PoseVectorSize = AutoEncoder->GetPoseVectorSize();

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingPoseVectors", "Computing Pose Vectors"));
					SlowTask.MakeDialog();

					PoseVectors = Learning::SharedMemory::Allocate<2, float>({ TotalFrameNum, PoseVectorSize });

					UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [this, AutoEncoder, &PoseData](const int32 Start, const int32 Length)
					{
						AutoEncoder->ToPoseVectors(PoseVectors.View.Slice(Start, Length), PoseData.ConstSlice(Start, Length));
					});
				}
				
				// Compute Pose Vector Normalization

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingNormalization", "Computing Normalization"));
					SlowTask.MakeDialog();

					AutoEncoder->PoseVectorOffset.SetNumUninitialized(PoseVectorSize);
					AutoEncoder->PoseVectorScale.SetNumUninitialized(PoseVectorSize);

					AnimDatabase::PoseData::FitPoseVectorNormalization(
						AutoEncoder->PoseVectorOffset,
						AutoEncoder->PoseVectorScale,
						PoseVectors.View,
						BoneLocationNum,
						BoneRotationNum,
						BoneScaleNum,
						AutoEncoder->AttributeTypes);

					AutoEncoder->PoseVectorMin.SetNumUninitialized(PoseVectorSize);
					AutoEncoder->PoseVectorMax.SetNumUninitialized(PoseVectorSize);

					UE::Learning::SlicedParallelFor(PoseVectorSize, 64, [this, AutoEncoder](const int32 Start, const int32 Length)
						{
							AnimDatabase::Math::ComputeMinMax(
								AutoEncoder->PoseVectorMin,
								AutoEncoder->PoseVectorMax,
								PoseVectors.View,
								Start,
								Length);
						});
				}

				// Normalize Pose Vectors

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("NormalizingPoseVectors", "Normalizing Pose Vectors"));
					SlowTask.MakeDialog();

					UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [this, AutoEncoder](const int32 Start, const int32 Length)
						{
							AutoEncoder->NormalizePoseVectors(PoseVectors.View.Slice(Start, Length));
						});
				}

				// Compute Normalized Mean and Std

				TLearningArray<1, float> PoseVectorNormalizedMean, PoseVectorNormalizedStd;
				PoseVectorNormalizedMean.SetNumUninitialized({ PoseVectorSize });
				PoseVectorNormalizedStd.SetNumUninitialized({ PoseVectorSize });

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingNormalizedMeanStd", "Computing Normalized Mean and Std"));
					SlowTask.MakeDialog();

					UE::Learning::SlicedParallelFor(PoseVectorSize, 64, [this, &PoseVectorNormalizedMean, &PoseVectorNormalizedStd](const int32 Start, const int32 Length)
						{
							AnimDatabase::Math::ComputeMeanStd(
								PoseVectorNormalizedMean,
								PoseVectorNormalizedStd,
								PoseVectors.View,
								Start,
								Length);

							for (int32 DimIdx = 0; DimIdx < Length; DimIdx++)
							{
								PoseVectorNormalizedStd[Start + DimIdx] = 
									PoseVectorNormalizedStd[Start + DimIdx] > UE_SMALL_NUMBER ? 
									FMath::Max(PoseVectorNormalizedStd[Start + DimIdx], UE_KINDA_SMALL_NUMBER) : 0.0f;
							}
						});
				}

				// Compute Pose Vector Weights

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingWeights", "Computing Pose Vector Weights"));
					SlowTask.MakeDialog();

					PoseVectorWeights = Learning::SharedMemory::Allocate<1, float>({ PoseVectorSize });

					AnimDatabase::PoseData::ComputePoseVectorWeightsCylinderApprox(
						PoseVectorWeights.View,
						TrainingDatabase->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose(),
						AutoEncoder->GetBoneParents(),
						AutoEncoder->AutoEncodedRequiredBoneIndices,
						AutoEncoder->AutoEncodedBoneLocationIndices,
						AutoEncoder->AutoEncodedBoneRotationIndices,
						AutoEncoder->AutoEncodedBoneScaleIndices,
						AutoEncoder->TrainingSettings->RootWeightMultiplier,
						AutoEncoder->TrainingSettings->BaseWeightMultiplier,
						AutoEncoder->TrainingSettings->AttributeWeightMultiplier);
				}

				// Controls

				TrainProcessControls = Learning::SharedMemory::Allocate<1, volatile int32>({ TrainControlNum });
				Learning::Array::Zero(TrainProcessControls.View);

				// Create Networks

				const int32 EncodingSize = AutoEncoder->TrainingSettings->EncodingSize;

				NNE::RuntimeBasic::FModelBuilder::EActivationFunction BuilderActivationFunction = NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
				switch (AutoEncoder->TrainingSettings->ActivationFunction)
				{
				case EAnimGenActivationFunction::ReLU: BuilderActivationFunction = NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU; break;
				case EAnimGenActivationFunction::ELU: BuilderActivationFunction = NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ELU; break;
				case EAnimGenActivationFunction::TanH: BuilderActivationFunction = NNE::RuntimeBasic::FModelBuilder::EActivationFunction::TanH; break;
				case EAnimGenActivationFunction::GELU: BuilderActivationFunction = NNE::RuntimeBasic::FModelBuilder::EActivationFunction::GELU; break;
				default: checkNoEntry();
				}

				NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
				LinearLayerSettings.Type = AutoEncoder->TrainingSettings->bUseCompressedLinearLayers ?
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

				switch (AutoEncoder->TrainingSettings->WeightInit)
				{
				case EAnimGenWeightInit::Gaussian: LinearLayerSettings.WeightInitializationSettings.Type = NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
				case EAnimGenWeightInit::Uniform: LinearLayerSettings.WeightInitializationSettings.Type = NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
				default: checkNoEntry();
				}

				// Make Encoder

				NNE::RuntimeBasic::FModelBuilder EncoderBuilder(Learning::Random::Int(AutoEncoder->TrainingSettings->RandomSeed ^ 0x133247e3));

				int32 EncoderCompatibilityHash = 0; // TODO
				TArray<uint8> EncoderFileData;
				uint32 EncoderInputSize, EncoderOutputSize;

				EncoderBuilder.WriteFileDataAndReset(
					EncoderFileData, EncoderInputSize, EncoderOutputSize,
					EncoderBuilder.MakeSequence({
						EncoderBuilder.MakeMLP(
							PoseVectorSize,
							EncodingSize,
							AutoEncoder->TrainingSettings->HiddenUnitNum,
							AutoEncoder->TrainingSettings->LayerNum,
							BuilderActivationFunction,
							false,
							LinearLayerSettings),
						EncoderBuilder.MakeTanH(EncodingSize)
					}));

				const FName EncoderName = MakeUniqueObjectName(AutoEncoder, ULearningNeuralNetworkData::StaticClass(), TEXT("Encoder"));
				AutoEncoder->EncoderNetwork = NewObject<ULearningNeuralNetworkData>(AutoEncoder, EncoderName);
				AutoEncoder->EncoderNetwork->Init(EncoderInputSize, EncoderOutputSize, EncoderCompatibilityHash, EncoderFileData);
				EncoderFileData.Empty();

				// Make Decoder

				NNE::RuntimeBasic::FModelBuilder DecoderBuilder(Learning::Random::Int(AutoEncoder->TrainingSettings->RandomSeed ^ 0x26e9bbbc));

				int32 DecoderCompatibilityHash = 0; // TODO
				TArray<uint8> DecoderFileData;
				uint32 DecoderInputSize, DecoderOutputSize;

				DecoderBuilder.WriteFileDataAndReset(
					DecoderFileData, DecoderInputSize, DecoderOutputSize,
					DecoderBuilder.MakeSequence({
						DecoderBuilder.MakeMLP(
							EncodingSize,
							PoseVectorSize,
							AutoEncoder->TrainingSettings->HiddenUnitNum,
							AutoEncoder->TrainingSettings->LayerNum,
							BuilderActivationFunction,
							false,
							LinearLayerSettings),
						DecoderBuilder.MakeDenormalize(
							PoseVectorSize,
							PoseVectorNormalizedMean,
							PoseVectorNormalizedStd)
					}));

				const FName DecoderName = MakeUniqueObjectName(AutoEncoder, ULearningNeuralNetworkData::StaticClass(), TEXT("Decoder"));
				AutoEncoder->DecoderNetwork = NewObject<ULearningNeuralNetworkData>(AutoEncoder, DecoderName);
				AutoEncoder->DecoderNetwork->Init(DecoderInputSize, DecoderOutputSize, DecoderCompatibilityHash, DecoderFileData);
				DecoderFileData.Empty();

				// Write Networks

				const int32 EncoderByteNum = AutoEncoder->EncoderNetwork->GetSnapshotByteNum();
				EncoderNetworkData = Learning::SharedMemory::Allocate<1, uint8>({ EncoderByteNum });
				AutoEncoder->EncoderNetwork->SaveToSnapshot(MakeArrayView<uint8>(EncoderNetworkData.View.GetData(), EncoderNetworkData.View.Num()));
				AutoEncoder->EncoderSize = AutoEncoder->EncoderNetwork->GetSnapshotByteNum() / 1000;

				const int32 DecoderByteNum = AutoEncoder->DecoderNetwork->GetSnapshotByteNum();
				DecoderNetworkData = Learning::SharedMemory::Allocate<1, uint8>({ DecoderByteNum });
				AutoEncoder->DecoderNetwork->SaveToSnapshot(MakeArrayView<uint8>(DecoderNetworkData.View.GetData(), DecoderNetworkData.View.Num()));
				AutoEncoder->DecoderSize = AutoEncoder->DecoderNetwork->GetSnapshotByteNum() / 1000;

				// Refresh UI

				AutoEncoder->TrainingSettings->ForceRefresh();
				ToolkitPtr->TrainingSettingsWidget->ForceRefresh();
				ToolkitPtr->EditingAssetWidget->ForceRefresh();

				// Write Config File

				const FString EnginePath = FPaths::EngineDir();
				const FString SitePackagesPath = Learning::Trainer::GetSitePackagesPath(EnginePath);
				const FString IntermediatePath = FPaths::ProjectIntermediateDir() / TEXT("AnimGen");
				const FString TaskName = AutoEncoder->GetName();
				const FString TrainerType = TEXT("AutoEncoder");

				IFileManager& FileManager = IFileManager::Get();
				const FString TimeStamp = FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S"));
				const FString ConfigPath = IntermediatePath / TEXT("Configs") / FString::Printf(TEXT("%s_%s_%s.json"), *TaskName, *TrainerType, *TimeStamp);

				TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();
				ConfigObject->SetStringField(TEXT("TaskName"), *TaskName);
				ConfigObject->SetStringField(TEXT("TrainerType"), TrainerType);
				ConfigObject->SetStringField(TEXT("TimeStamp"), *TimeStamp);

				ConfigObject->SetStringField(TEXT("EnginePath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*EnginePath));
				ConfigObject->SetStringField(TEXT("SitePackagesPath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*SitePackagesPath));
				ConfigObject->SetStringField(TEXT("IntermediatePath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*IntermediatePath));

				ConfigObject->SetNumberField(TEXT("RangeNum"), TotalRangeNum);
				ConfigObject->SetNumberField(TEXT("TotalFrameNum"), TotalFrameNum);
				ConfigObject->SetNumberField(TEXT("PoseVectorSize"), PoseVectorSize);
				ConfigObject->SetNumberField(TEXT("EncoderByteNum"), EncoderByteNum);
				ConfigObject->SetNumberField(TEXT("DecoderByteNum"), DecoderByteNum);
				ConfigObject->SetNumberField(TEXT("DeltaTime"), 1.0f / TrainingDatabase->GetFrameRate().AsDecimal());
				ConfigObject->SetNumberField(TEXT("EncodingSize"), EncodingSize);

				ConfigObject->SetNumberField(TEXT("IterationNum"), AutoEncoder->TrainingSettings->NumberOfIterations);
				ConfigObject->SetNumberField(TEXT("LearningRate"), AutoEncoder->TrainingSettings->LearningRate);
				ConfigObject->SetNumberField(TEXT("WarmupIterations"), AutoEncoder->TrainingSettings->WarmupIterations);
				ConfigObject->SetNumberField(TEXT("BatchSize"), AutoEncoder->TrainingSettings->BatchSize);
				ConfigObject->SetNumberField(TEXT("Seed"), AutoEncoder->TrainingSettings->RandomSeed);
				ConfigObject->SetStringField(TEXT("Device"), AutoEncoder->TrainingSettings->Device == EAnimGenTrainingDevice::GPU ? TEXT("GPU") : TEXT("CPU"));
				ConfigObject->SetBoolField(TEXT("EnableTensorboard"), AutoEncoder->TrainingSettings->bEnableTensorboard);

				ConfigObject->SetStringField(TEXT("ControlGuid"), *TrainProcessControls.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("EncoderGuid"), *EncoderNetworkData.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("DecoderGuid"), *DecoderNetworkData.GetPlatformGuidString());

				ConfigObject->SetStringField(TEXT("RangeStartsGuid"), *RangeStarts.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("RangeLengthsGuid"), *RangeLengths.GetPlatformGuidString());

				ConfigObject->SetStringField(TEXT("PoseVectorsGuid"), *PoseVectors.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("PoseVectorWeightsGuid"), *PoseVectorWeights.GetPlatformGuidString());

				FString JsonString;
				TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
				FJsonSerializer::Serialize(ConfigObject, JsonWriter, true);

				FFileHelper::SaveStringToFile(JsonString, *ConfigPath);

				// Reset Iterations

				MaxIterations = AutoEncoder->TrainingSettings->NumberOfIterations;
				Iteration = 0;
				StartTrainingTimeIteration = -1;

				// Launch Training Process

				const FString PythonScriptPath = FPaths::EngineDir() / TEXT("Plugins/Experimental/Animation/AnimGen/Content/Python/train_autoencoder.py");
				const FString PythonExecutablePath = Learning::Trainer::GetPythonExecutablePath(FPaths::ProjectIntermediateDir());

				const FString CommandLineArguments = FString::Printf(TEXT("\"%s\" \"%s\""),
					*FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonScriptPath),
					*FileManager.ConvertToAbsolutePathForExternalAppForRead(*ConfigPath));

				UE::Learning::ESubprocessFlags SubprocessFlags = UE::Learning::ESubprocessFlags::None;
				if (!AutoEncoder->TrainingSettings->bEnableLogging) { SubprocessFlags |= UE::Learning::ESubprocessFlags::NoRedirectOutput; }

				check(!TrainProcess.IsRunning());
				TrainProcess.Launch(
					FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonExecutablePath),
					CommandLineArguments,
					SubprocessFlags);

				ProgressNotification = FSlateNotificationManager::Get().StartProgressNotification(
					FText::Format(LOCTEXT("AutoEncoderProgressTrainingMessage", "Training {0}"), FText::FromString(AutoEncoder->GetName())),
					MaxIterations);

				bTrainingRunning = true;
			}
		}

		virtual void StopTraining() override
		{
			if (!IsTraining()) { return; }

			TrainProcessControls.View[TrainControlExit] = 1;

			float Timer = 0.0f;
			const float SleepDuration = 0.01f;
			const float Timeout = 5.0f;

			while (TrainProcess.Update())
			{
				if (Timer > Timeout)
				{
					TrainProcess.Terminate();
					break;
				}

				Timer += SleepDuration;
				FPlatformProcess::Sleep(SleepDuration);
			}

			DoneTraining();
		}

		void DoneTraining()
		{
			bTrainingRunning = false;
			TrainProcess.Terminate();
			if (ProgressNotification.IsValid()) { FSlateNotificationManager::Get().CancelProgressNotification(ProgressNotification); }
			ProgressNotification.Reset();
			MaxIterations = 0;
			Iteration = 0;
			StartTrainingTimeIteration = -1;

			Learning::SharedMemory::Deallocate(TrainProcessControls);

			Learning::SharedMemory::Deallocate(EncoderNetworkData);
			Learning::SharedMemory::Deallocate(DecoderNetworkData);

			Learning::SharedMemory::Deallocate(RangeStarts);
			Learning::SharedMemory::Deallocate(RangeLengths);

			Learning::SharedMemory::Deallocate(PoseVectors);
			Learning::SharedMemory::Deallocate(PoseVectorWeights);
		}

		virtual bool IsTraining() const override { return bTrainingRunning; }

		virtual int32 GetProgressBarNum() const override { return 1; }

		virtual ETrainingStatus GetTrainingStatus(const int32 ProgressBarIdx) const override
		{
			if (IsTraining() && Iteration == 0)
			{
				return ETrainingStatus::Preparing;
			}
			else if (IsTraining() && Iteration > 0)
			{
				return ETrainingStatus::Training;
			}
			else if (!IsTraining() && Toolkit.IsValid() && Toolkit.Pin()->AutoEncoder->IsValid())
			{
				return ETrainingStatus::Done;
			}
			else
			{
				return ETrainingStatus::NotStarted;
			}
		}

		virtual FText GetErrorMessage() const override
		{
			return ErrorMessage;
		}

		virtual int32 GetIterationNum(const int32 ProgressBarIdx) const override
		{
			return Iteration;
		}

		virtual int32 GetMaxIterationNum(const int32 ProgressBarIdx) const override
		{
			return MaxIterations;
		}

		virtual float GetTrainingLoss(const int32 ProgressBarIdx) const override
		{
			return IsTraining() ? ((float)TrainProcessControls.View[TrainControlLoss]) / 100000.0f : 0.0f;
		}

		virtual TOptional<FTimespan> GetEstimateTimeRemaining(const int32 ProgressBarIdx) const override
		{
			if (StartTrainingTimeIteration != -1 && Iteration > StartTrainingTimeIteration + 250)
			{
				const FTimespan ElapsedTime = FDateTime::Now() - StartTrainingTime;
				const int32 ElapsedIteration = Iteration - StartTrainingTimeIteration;
				const int32 RemainingIteration = MaxIterations - Iteration;
				const float IterationRatio = (float)RemainingIteration / (float)ElapsedIteration;
				return IterationRatio * ElapsedTime;
			}

			return TOptional<FTimespan>();
		}

		void Update()
		{
			if (TSharedPtr<FAutoEncoderToolkit> ToolkitPtr = Toolkit.Pin())
			{
				UAnimGenAutoEncoder* AutoEncoder = ToolkitPtr->AutoEncoder;

				if (!bTrainingRunning) { return; }

				if (TrainProcess.Update())
				{
					Iteration = TrainProcessControls.View[TrainControlIteration];

					// Update Estimated Time

					if (Iteration > 250 && StartTrainingTimeIteration == -1)
					{
						StartTrainingTimeIteration = Iteration;
						StartTrainingTime = FDateTime::Now();
					}

					// Update Progress Notification

					if (ProgressNotification.IsValid())
					{
						FSlateNotificationManager::Get().UpdateProgressNotification(ProgressNotification, Iteration + 1);
					}

					// Update Networks

					if (AutoEncoder->EncoderNetwork &&
						EncoderNetworkData.View.GetData() &&
						TrainProcessControls.View[TrainControlEncoderNetworkUpdate])
					{
						AutoEncoder->EncoderNetwork->LoadFromSnapshot(EncoderNetworkData.View);
						AutoEncoder->Modify();
						TrainProcessControls.View[TrainControlEncoderNetworkUpdate] = 0;
					}

					if (AutoEncoder->DecoderNetwork &&
						DecoderNetworkData.View.GetData() &&
						TrainProcessControls.View[TrainControlDecoderNetworkUpdate])
					{
						AutoEncoder->DecoderNetwork->LoadFromSnapshot(DecoderNetworkData.View);
						AutoEncoder->Modify();
						TrainProcessControls.View[TrainControlDecoderNetworkUpdate] = 0;
					}
				}
				else
				{
					DoneTraining();
				}
			}
		}

		static constexpr int32 TrainControlExit = 0;
		static constexpr int32 TrainControlIteration = 1;
		static constexpr int32 TrainControlLoss = 2;
		static constexpr int32 TrainControlEncoderNetworkUpdate = 3;
		static constexpr int32 TrainControlDecoderNetworkUpdate = 4;
		static constexpr int32 TrainControlNum = 5;

		TWeakPtr<FAutoEncoderToolkit> Toolkit;

		Learning::TSharedMemoryArrayView<1, volatile int32> TrainProcessControls;

		Learning::TSharedMemoryArrayView<1, uint8> EncoderNetworkData;
		Learning::TSharedMemoryArrayView<1, uint8> DecoderNetworkData;

		Learning::TSharedMemoryArrayView<1, int32> RangeStarts;
		Learning::TSharedMemoryArrayView<1, int32> RangeLengths;

		Learning::TSharedMemoryArrayView<2, float> PoseVectors;
		Learning::TSharedMemoryArrayView<1, float> PoseVectorWeights;

		bool bTrainingRunning = false;
		Learning::FSubprocess TrainProcess;
		int32 Iteration = 0;
		int32 MaxIterations = 0;
		FText ErrorMessage;
		FProgressNotificationHandle ProgressNotification;
		int32 StartTrainingTimeIteration = -1;
		FDateTime StartTrainingTime;
	};


	static const FName AutoEncoderTabAssetDetails(TEXT("AnimGenEditorAutoEncoderAssetDetailsTabID"));
	static const FName AutoEncoderTabViewportSettings(TEXT("AnimGenEditorAutoEncoderViewportSettingsTabID"));
	static const FName AutoEncoderTabPreviewSettings(TEXT("AnimGenEditorAutoEncoderPreviewSettingsTabID"));
	static const FName AutoEncoderTabViewport(TEXT("AnimGenEditorAutoEncoderViewportTabID"));
	static const FName AutoEncoderTabTimeline(TEXT("AnimGenEditorAutoEncoderTimelineTabID"));
	static const FName AutoEncoderTabTrainingSettings(TEXT("AnimGenEditorAutoEncoderTrainingSettingsTabID"));
	static const FName AutoEncoderTabTraining(TEXT("AnimGenEditorAutoEncoderTrainingTabID"));

	void FAutoEncoderToolkit::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimGenAutoEncoder* InAutoEncoder)
	{
		AutoEncoder = InAutoEncoder;
		AutoEncoder->TrainingSettings->LoadDatabaseAsync();

		// Create Preview Scene
		PreviewScene = MakeShared<AnimDatabase::Editor::FPreviewScene>(AnimDatabase::Editor::FPreviewScene::ConstructionValues().ShouldSimulatePhysics(true).ForceUseMovementComponentInNonGameWorld(true));
		//Temporary fix for missing attached assets - MDW (Copied from FPersonaToolkit::CreatePreviewScene)
		PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);

		// Create Timeline Model
		TimelineModel = MakeShared<AnimDatabase::Editor::FTimelineModel>();

		// Create Tracks Model
		TracksModel = MakeShared<AnimDatabase::Editor::FTimelineTracksModel>();

		// Create Training Model
		TrainingModel = MakeShared<FAutoEncoderTrainingModel>(StaticCastWeakPtr<FAutoEncoderToolkit>(AsWeak()));

		// Create Widgets
		ViewportWidget = SNew(SAutoEncoderViewport, StaticCastSharedRef<FAutoEncoderToolkit>(AsShared()), PreviewScene.ToSharedRef(), FAutoEncoderMode::EditorModeId);
		TimelineWidget = SNew(AnimDatabase::Editor::STimeline, TimelineModel.ToWeakPtr(), TracksModel);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;

		EditingAssetWidget = PropertyModule.CreateDetailView(Args);
		EditingAssetWidget->SetObject(InAutoEncoder);

		ViewportSettingsWidget = PropertyModule.CreateDetailView(Args);
		ViewportSettingsWidget->SetObject(InAutoEncoder->ViewportSettings);

		FDetailsViewArgs TrainingSettingsArgs;
		TrainingSettingsArgs.bHideSelectionTip = true;
		TrainingSettingsArgs.bAllowSearch = false;

		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		PreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());

		TrainingSettingsWidget = PropertyModule.CreateDetailView(TrainingSettingsArgs);
		TrainingSettingsWidget->SetObject(InAutoEncoder->TrainingSettings);

		TrainingWidget = SNew(STraining, TrainingModel.ToWeakPtr());

		// Define Editor Layout
		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_AnimGenEditorAutoEncoder_Layout_v0.07")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.8f)
							->AddTab(AutoEncoderTabTrainingSettings, ETabState::OpenedTab)
							->SetForegroundTab(AutoEncoderTabTrainingSettings)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->AddTab(AutoEncoderTabTraining, ETabState::OpenedTab)
							->SetForegroundTab(AutoEncoderTabTraining)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.6f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.9f)
							->AddTab(AutoEncoderTabViewport, ETabState::OpenedTab)->SetHideTabWell(true)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)
							->AddTab(AutoEncoderTabTimeline, ETabState::OpenedTab)->SetHideTabWell(true)
						)
					)
					->Split(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(AutoEncoderTabAssetDetails, ETabState::OpenedTab)
							->AddTab(AutoEncoderTabPreviewSettings, ETabState::OpenedTab)
							->SetForegroundTab(AutoEncoderTabAssetDetails)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(AutoEncoderTabViewportSettings, ETabState::OpenedTab)
							->SetForegroundTab(AutoEncoderTabViewportSettings)
						)
					)
				)
			);

		const bool bCreateDefaultStandaloneMenuParam = true;
		const bool bCreateDefaultToolbarParam = true;
		const bool bIsToolbarFocusableParam = false;
		FAssetEditorToolkit::InitAssetEditor(
			Mode, InitToolkitHost, 
			TEXT("AnimGenEditorAutoEncoderApp"),
			StandaloneDefaultLayout, 
			bCreateDefaultStandaloneMenuParam, 
			bCreateDefaultToolbarParam, 
			InAutoEncoder,
			bIsToolbarFocusableParam);
	}

	void FAutoEncoderToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_AnimGenEditorAutoEncoder", "Animation Generation Auto Encoder Editor"));
		auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(AutoEncoderTabViewport, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&){
			return SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"))[ ViewportWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabAssetDetails, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("AssetDetailsTab_Title", "Asset Details"))[EditingAssetWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("AssetDetailsTab", "Asset Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabViewportSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ViewportSettingsTab_Title", "Viewport Settings"))[ViewportSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("ViewportSettingsTab", "Viewport Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabPreviewSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("PreviewSettingsTab_Title", "Preview Settings"))[PreviewSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("PreviewSettingsTab", "Preview Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabTimeline, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TimelineTab_Title", "Timeline"))[TimelineWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TimelineTab", "Timeline"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabTrainingSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TrainingSettingsTab_Title", "Training Settings"))[TrainingSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TrainingSettingsTab", "Training Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(AutoEncoderTabTraining, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TrainingTab_Title", "Training"))[TrainingWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TrainingTab", "Training"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));
	}

	void FAutoEncoderToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(AutoEncoderTabViewport);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabAssetDetails);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabViewportSettings);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabPreviewSettings);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabTimeline);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabTrainingSettings);
		InTabManager->UnregisterTabSpawner(AutoEncoderTabTraining);
	}

	FName FAutoEncoderToolkit::GetToolkitFName() const
	{
		return FName("AnimGenEditorAutoEncoder");
	}

	FText FAutoEncoderToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("AnimGenEditorAutoEncoderAppLabel", "AnimGen Auto-Encoder Editor");
	}

	FText FAutoEncoderToolkit::GetToolkitName() const
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::FromString(AutoEncoder->GetName()));
		return FText::Format(LOCTEXT("AnimGenEditorAutoEncoderToolkitName", "{AssetName}"), Args);
	}

	FLinearColor FAutoEncoderToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}

	FString FAutoEncoderToolkit::GetWorldCentricTabPrefix() const
	{
		return TEXT("AnimGenEditorAutoEncoder");
	}

	void FAutoEncoderToolkit::ReconstructTimelineWidget()
	{
		// Re-create the timeline widget
		TimelineWidget = SNew(AnimDatabase::Editor::STimeline, TimelineModel.ToWeakPtr(), TracksModel);

		// Switch out the content of the timeline tab to the new widget
		if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(AutoEncoderTabTimeline))
		{
			Tab->SetContent(TimelineWidget.ToSharedRef());
		}
	}

	void FAutoEncoderToolkit::Tick(float DeltaTime)
	{
		TrainingModel->Update();
	}

	TStatId FAutoEncoderToolkit::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimGenEditorToolkit, STATGROUP_Tickables);
	}
}

#undef LOCTEXT_NAMESPACE

