// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditorControllerMode.h"
#include "AnimGenEditorControllerToolkit.h"
#include "SAnimGenEditorControlObject.h"

#include "AnimDatabaseEditorViewportClient.h"
#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorTimeline.h"

#include "AnimGenLog.h"
#include "AnimGenController.h"
#include "AnimGenControl.h"
#include "AnimGenAutoEncoder.h"
#include "AnimGenBehavior.h"

#include "AnimDatabase.h"
#include "AnimDatabasePose.h"
#include "AnimDatabaseFrameAttribute.h"
#include "AnimDatabaseMath.h"

#include "LearningNeuralNetwork.h"
#include "LearningRandom.h"

#include "GameFramework/Character.h"
#include "Components/PoseableMeshComponent.h"
#include "Components/CapsuleComponent.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Async/ParallelFor.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "Preferences/PersonaOptions.h"
#include "DrawDebugLibrary.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorControllerMode"

namespace UE::AnimGen::Editor
{
	namespace Controller::Private
	{
		static inline FVector GetRangeOffset(
			const int32 CharacterIdx,
			const int32 CharacterNum,
			const float Spacing,
			const bool bOffsetRanges,
			const bool bOffsetRangesInGrid)
		{
			const int32 GridRows = FMath::CeilToInt(FMath::Sqrt((float)CharacterNum));

			return
				bOffsetRanges ?
				bOffsetRangesInGrid ?
				FVector(
					Spacing * (CharacterIdx % GridRows) - 0.5f * Spacing * (GridRows - 1),
					Spacing * (CharacterIdx / GridRows) - 0.5f * Spacing * (GridRows - 1), 0.0f) :
				FVector(
					Spacing * CharacterIdx - 0.5f * Spacing * (CharacterNum - 1), 0.0f, 0.0f) :
				FVector::ZeroVector;
		}

		static inline void AdjustTransformRootTransform(
			FVector& InOutLocation,
			FQuat4f& InOutRotation,
			FVector3f& InOutLinearVelocity,
			FVector3f& InOutAngularVelocity,
			const FVector Offset,
			const FTransform RelativeTransform,
			const bool bDisableRootTranslation,
			const bool bDisableRootRotation)
		{
			const FTransform RootOffsetTransform = FTransform(FQuat::Identity, Offset, FVector::OneVector);
			const FTransform RootAdjustmenTransform = RelativeTransform.Inverse() * RootOffsetTransform;

			if (bDisableRootTranslation && bDisableRootRotation)
			{
				const FQuat4f RootRotationOffset = InOutRotation;
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = FQuat4f::Identity;
				InOutLinearVelocity = RootRotationOffset.UnrotateVector(InOutLinearVelocity);
				InOutAngularVelocity = RootRotationOffset.UnrotateVector(InOutAngularVelocity);
			}
			else if (bDisableRootTranslation)
			{
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
			else
			{
				InOutLocation = RootAdjustmenTransform.TransformPosition(InOutLocation);
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
		}

		static inline bool FindControlSetsTotalRange(int32& OutControlSetIdx, int32& OutRangeEntry, int32& OutRangeRange, const TArrayView<const FAnimGenControlSet> ControlSets, const int32 TotalRangeIdx)
		{
			const int32 ControlSetNum = ControlSets.Num();
			int32 ControlSetRangeOffset = 0;
			for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
			{
				const int32 ControlSetRangeNum = ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalRangeNum();

				if (TotalRangeIdx >= ControlSetRangeOffset && TotalRangeIdx < ControlSetRangeOffset + ControlSetRangeNum)
				{
					OutControlSetIdx = ControlSetIdx;
					return ControlSets[ControlSetIdx].Data->FrameRangeSet.FindTotalRange(OutRangeEntry, OutRangeRange, TotalRangeIdx - ControlSetRangeOffset);
				}

				ControlSetRangeOffset += ControlSetRangeNum;
			}

			OutControlSetIdx = INDEX_NONE;
			OutRangeEntry = INDEX_NONE;
			OutRangeRange = INDEX_NONE;
			return false;
		}

		static bool UpdateControlObjectTreeElement(
			TSharedPtr<FControlObjectTreeElement> TreeElement,
			const Learning::Observation::FObject& ControlObject, 
			const Learning::Observation::FObjectElement& ControlObjectElement)
		{
			if (!ControlObject.IsValid(ControlObjectElement))
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::Invalid;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->Tag = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}

			TreeElement->Tag = ControlObject.GetTag(ControlObjectElement);

			switch (ControlObject.GetType(ControlObjectElement))
			{
			case Learning::Observation::EType::Null:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::Null;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}

			case Learning::Observation::EType::Continuous:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::Continuous;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues = ControlObject.GetContinuous(ControlObjectElement).Values;
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}

			case Learning::Observation::EType::DiscreteExclusive:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::DiscreteExclusive;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->NamedValues.Empty();
				TreeElement->DiscreteValues.Empty(1);
				TreeElement->DiscreteValues.Add(ControlObject.GetDiscreteExclusive(ControlObjectElement).DiscreteIndex);
				return bHasChildren;
			}

			case Learning::Observation::EType::DiscreteInclusive:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::DiscreteInclusive;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues = ControlObject.GetDiscreteInclusive(ControlObjectElement).DiscreteIndices;
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}

			case Learning::Observation::EType::NamedDiscreteExclusive:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::NamedDiscreteExclusive;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty(1);
				TreeElement->NamedValues.Add(ControlObject.GetNamedDiscreteExclusive(ControlObjectElement).ElementName);
				return bHasChildren;
			}

			case Learning::Observation::EType::NamedDiscreteInclusive:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::NamedDiscreteInclusive;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues = ControlObject.GetNamedDiscreteInclusive(ControlObjectElement).ElementNames;
				return bHasChildren;
			}

			case Learning::Observation::EType::And:
			{
				TreeElement->Type = EControlObjectTreeElementType::And;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const TArrayView<const Learning::Observation::FObjectElement> SubElements = ControlObject.GetAnd(ControlObjectElement).Elements;
				const TArrayView<const FName> SubElementNames = ControlObject.GetAnd(ControlObjectElement).ElementNames;
				const int32 SubElementNum = SubElements.Num();

				bool bRefresh = TreeElement->Children.Num() != SubElements.Num();
				TreeElement->Children.SetNum(SubElements.Num());

				for (int32 SubElementIdx = 0; SubElementIdx < SubElementNum; SubElementIdx++)
				{
					if (!TreeElement->Children[SubElementIdx].IsValid())
					{
						TreeElement->Children[SubElementIdx] = MakeShared<FControlObjectTreeElement>();
						bRefresh = true;
					}

					bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[SubElementIdx], ControlObject, SubElements[SubElementIdx]);
					TreeElement->Children[SubElementIdx]->Name = SubElementNames[SubElementIdx];
				}

				return bRefresh;
			}

			case Learning::Observation::EType::OrExclusive:
			{
				TreeElement->Type = EControlObjectTreeElementType::OrExclusive;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const Learning::Observation::FObjectElement SubElement = ControlObject.GetOrExclusive(ControlObjectElement).Element;
				const FName SubElementName = ControlObject.GetOrExclusive(ControlObjectElement).ElementName;
				
				bool bRefresh = TreeElement->Children.Num() != 1;
				TreeElement->Children.SetNum(1);
				
				if (!TreeElement->Children[0].IsValid())
				{
					TreeElement->Children[0] = MakeShared<FControlObjectTreeElement>();
					bRefresh = true;
				}

				bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[0], ControlObject, SubElement);
				TreeElement->Children[0]->Name = SubElementName;

				return bRefresh;
			}

			case Learning::Observation::EType::OrInclusive:
			{
				TreeElement->Type = EControlObjectTreeElementType::OrInclusive;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const TArrayView<const Learning::Observation::FObjectElement> SubElements = ControlObject.GetOrInclusive(ControlObjectElement).Elements;
				const TArrayView<const FName> SubElementNames = ControlObject.GetOrInclusive(ControlObjectElement).ElementNames;
				const int32 SubElementNum = SubElements.Num();

				bool bRefresh = TreeElement->Children.Num() != SubElements.Num();
				TreeElement->Children.SetNum(SubElements.Num());

				for (int32 SubElementIdx = 0; SubElementIdx < SubElementNum; SubElementIdx++)
				{
					if (!TreeElement->Children[SubElementIdx].IsValid())
					{
						TreeElement->Children[SubElementIdx] = MakeShared<FControlObjectTreeElement>();
						bRefresh = true;
					}

					bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[SubElementIdx], ControlObject, SubElements[SubElementIdx]);
					TreeElement->Children[SubElementIdx]->Name = SubElementNames[SubElementIdx];
				}

				return bRefresh;
			}

			case Learning::Observation::EType::Array:
			{
				TreeElement->Type = EControlObjectTreeElementType::Array;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const TArrayView<const Learning::Observation::FObjectElement> SubElements = ControlObject.GetArray(ControlObjectElement).Elements;
				const int32 SubElementNum = SubElements.Num();

				bool bRefresh = TreeElement->Children.Num() != SubElements.Num();
				TreeElement->Children.SetNum(SubElements.Num());

				for (int32 SubElementIdx = 0; SubElementIdx < SubElementNum; SubElementIdx++)
				{
					if (!TreeElement->Children[SubElementIdx].IsValid())
					{
						TreeElement->Children[SubElementIdx] = MakeShared<FControlObjectTreeElement>();
						bRefresh = true;
					}

					bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[SubElementIdx], ControlObject, SubElements[SubElementIdx]);
				}

				return bRefresh;
			}

			case Learning::Observation::EType::Set:
			{
				TreeElement->Type = EControlObjectTreeElementType::Set;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const TArrayView<const Learning::Observation::FObjectElement> SubElements = ControlObject.GetSet(ControlObjectElement).Elements;
				const int32 SubElementNum = SubElements.Num();

				bool bRefresh = TreeElement->Children.Num() != SubElements.Num();
				TreeElement->Children.SetNum(SubElements.Num());

				for (int32 SubElementIdx = 0; SubElementIdx < SubElementNum; SubElementIdx++)
				{
					if (!TreeElement->Children[SubElementIdx].IsValid())
					{
						TreeElement->Children[SubElementIdx] = MakeShared<FControlObjectTreeElement>();
						bRefresh = true;
					}

					bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[SubElementIdx], ControlObject, SubElements[SubElementIdx]);
				}

				return bRefresh;
			}

			case Learning::Observation::EType::Encoding:
			{
				TreeElement->Type = EControlObjectTreeElementType::Encoding;
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();

				const Learning::Observation::FObjectElement SubElement = ControlObject.GetEncoding(ControlObjectElement).Element;

				bool bRefresh = TreeElement->Children.Num() != 1;
				TreeElement->Children.SetNum(1);

				if (!TreeElement->Children[0].IsValid())
				{
					TreeElement->Children[0] = MakeShared<FControlObjectTreeElement>();
					bRefresh = true;
				}

				bRefresh |= UpdateControlObjectTreeElement(TreeElement->Children[0], ControlObject, SubElement);

				return bRefresh;
			}

			case Learning::Observation::EType::Sparse:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::Sparse;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues = ControlObject.GetSparse(ControlObjectElement).Values;
				TreeElement->DiscreteValues = ControlObject.GetSparse(ControlObjectElement).DiscreteIndices;
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}

			case Learning::Observation::EType::NamedSparse:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::NamedSparse;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues = ControlObject.GetNamedSparse(ControlObjectElement).Values;
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues = ControlObject.GetNamedSparse(ControlObjectElement).ElementNames;
				return bHasChildren;
			}

			default:
			{
				const bool bHasChildren = !TreeElement->Children.IsEmpty();
				TreeElement->Type = EControlObjectTreeElementType::Invalid;
				TreeElement->Children.Empty();
				TreeElement->Name = NAME_None;
				TreeElement->ContinuousValues.Empty();
				TreeElement->DiscreteValues.Empty();
				TreeElement->NamedValues.Empty();
				return bHasChildren;
			}
			}
		}
	}

	const FEditorModeID FControllerMode::EditorModeId = TEXT("AnimGenEditorControllerMode");

	FControllerMode::FControllerMode() = default;
	FControllerMode::~FControllerMode() = default;

	void FControllerMode::Tick(FEditorViewportClient* EditorViewportClient, float DeltaTime)
	{
		FEdMode::Tick(EditorViewportClient, DeltaTime);

		AnimDatabase::Editor::FViewportClient* ViewportClient = static_cast<AnimDatabase::Editor::FViewportClient*>(EditorViewportClient);
		check(ViewportClient);
		ViewportClient->ClearWarningMessage();

		TSharedPtr<FControllerToolkit> EditorToolKit = StaticCastSharedPtr<FControllerToolkit>(ViewportClient->GetAssetEditorToolkit().Pin());
		if (!EditorToolKit.IsValid()) { return; }

		WeakToolkit = EditorToolKit.ToWeakPtr();

		TSharedPtr<UE::AnimDatabase::Editor::FPreviewScene> PreviewScene = ViewportClient->GetPreviewScene().Pin();
		if (!PreviewScene.IsValid()) { return; }

		// Get the Controller

		UAnimGenController* Controller = EditorToolKit->Controller;
		check(Controller);

		// Update Training Settings

		Controller->TrainingSettings->RangeIndentifierColorSeed = Controller->ViewportSettings->RangeIdentifierColorSeed;
		const bool bTrainingSettingsUpdated = Controller->TrainingSettings->Update();

		if (bTrainingSettingsUpdated)
		{
			ActiveRanges.Reset();
			EditorToolKit->TrainingSettingsWidget->ForceRefresh();
		}

		// Load Database

		UAnimDatabase* TrainingDatabase = Controller->TrainingSettings->Database.Get();
		if (!TrainingDatabase)
		{
			ViewportClient->SetWarningMessage(
				Controller->TrainingSettings->Database.IsNull() ?
				LOCTEXT("NoQuery", "Warning: No Training Database Provided.") :
				LOCTEXT("LoadingDatabase", "Loading Training Database..."));
			ActiveRanges.Reset();
			return;
		}

		// Load AutoEncoder

		UAnimGenAutoEncoder* TrainingAutoEncoder = Controller->TrainingSettings->AutoEncoder.Get();
		if (!TrainingAutoEncoder)
		{
			ViewportClient->SetWarningMessage(
				LOCTEXT("LoadingAutoEncoder", "Loading Training AutoEncoder..."));
			ActiveRanges.Reset();
			return;
		}

		if (!TrainingAutoEncoder->IsValid())
		{
			ViewportClient->SetWarningMessage(
				LOCTEXT("AutoEncoderUntrained", "AutoEncoder not Trained..."));
			ActiveRanges.Reset();
			return;
		}

		// Add the required number of characters to the scene

		while (PreviewScene->GetCharacterNum() < MaxCharacterNum)
		{
			const int32 CharacterIdx = PreviewScene->AddCharacter();
			PreviewScene->SetCharacterVisibility(CharacterIdx, false);
		}

		// Set the character's skeletal mesh based on what is given in the animation database

		USkinnedAsset* DesiredSkeletalMesh = TrainingDatabase->Skeleton ? TrainingDatabase->Skeleton->GetPreviewMesh() : nullptr;
		DesiredSkeletalMesh = Controller->ViewportSettings->PreviewMesh ? Controller->ViewportSettings->PreviewMesh.Get() : DesiredSkeletalMesh;

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			USkinnedAsset* CurrentSkeletalMesh = PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset();

			if (DesiredSkeletalMesh && DesiredSkeletalMesh != CurrentSkeletalMesh)
			{
				PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetSkinnedAssetAndUpdate(DesiredSkeletalMesh, true);
			}
		}

		// Compute all range trajectories

		int32 QueryRangeNum = 0;

		const int32 ControlSetNum = Controller->TrainingSettings->ControlSets.Num();
		for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
		{
			QueryRangeNum += Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalRangeNum();
		}

		if (bTrainingSettingsUpdated || RangeTrajectoryLocations.IsEmpty())
		{
			RangeTrajectoryLocations.SetNum(QueryRangeNum);
			RangeTrajectoryRotations.SetNum(QueryRangeNum);

			ParallelFor(QueryRangeNum, [this, TrainingDatabase, Controller](int32 RangeIdx)
				{
					int32 ControlSetIdx = INDEX_NONE, RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
					const bool bFound = Controller::Private::FindControlSetsTotalRange(ControlSetIdx, RangeEntry, RangeRange, Controller->TrainingSettings->ControlSets, RangeIdx);
					check(bFound);

					const int32 RangeFrameNum = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntryRangeLength(RangeEntry, RangeRange);
					const int32 RangeSequence = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntrySequence(RangeEntry);
					const int32 RangeStart = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntryRangeStart(RangeEntry, RangeRange);

					RangeTrajectoryLocations[RangeIdx].SetNumUninitialized({ RangeFrameNum });
					RangeTrajectoryRotations[RangeIdx].SetNumUninitialized({ RangeFrameNum });

					TrainingDatabase->GetRootLocation(RangeTrajectoryLocations[RangeIdx], RangeSequence, RangeStart);
					TrainingDatabase->GetRootRotation(RangeTrajectoryRotations[RangeIdx], RangeSequence, RangeStart);
				});
		}

		// Adjust Number of visible characters in the scene

		CharacterNum = FMath::Min(Controller->TrainingSettings->SelectedRanges.Num(), MaxCharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			PreviewScene->SetCharacterVisibility(CharacterIdx, CharacterIdx < CharacterNum);
		}

		// If selection has changed adjust view ranges

		if (bTrainingSettingsUpdated || ActiveRanges != Controller->TrainingSettings->SelectedRanges)
		{
			ActiveRanges = Controller->TrainingSettings->SelectedRanges;

			// Add error message if too many ranges selected

			if (ActiveRanges.Num() > MaxCharacterNum)
			{
				ViewportClient->SetWarningMessage(FText::Format(
					LOCTEXT("AnimGenTooManyRangesWarning", "Warning: Rendering characters for {0} for {1} selected ranges."),
					FText::AsNumber(MaxCharacterNum),
					FText::AsNumber(ActiveRanges.Num())));
			}

			// Adjust the timeline for the newly selected ranges

			if (ActiveRanges.Num() == 0)
			{
				EditorToolKit->TimelineModel->SetTimelineFrameRate(TrainingDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
			else if (ActiveRanges.Num() == 1)
			{
				int32 ControlSetIdx = INDEX_NONE, RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
				const bool bFound = Controller::Private::FindControlSetsTotalRange(ControlSetIdx, RangeEntry, RangeRange, Controller->TrainingSettings->ControlSets, ActiveRanges[0]);
				check(bFound);

				const int32 RangeStart = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntryRangeStart(RangeEntry, RangeRange);
				const int32 RangeLength = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntryRangeLength(RangeEntry, RangeRange);

				EditorToolKit->TimelineModel->SetTimelineFrameRate(TrainingDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(RangeStart));
			}
			else
			{
				int32 RangeLength = 1;
				for (const int32 ActiveRangeIdx : ActiveRanges)
				{
					int32 ControlSetIdx = INDEX_NONE, RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
					const bool bFound = Controller::Private::FindControlSetsTotalRange(ControlSetIdx, RangeEntry, RangeRange, Controller->TrainingSettings->ControlSets, ActiveRangeIdx);
					check(bFound);
					RangeLength = FMath::Max(Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet.GetEntryRangeLength(RangeEntry, RangeRange), RangeLength);
				}

				EditorToolKit->TimelineModel->SetTimelineFrameRate(TrainingDatabase->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
		}

		// Update Character Range Data

		CharacterSequenceIndices.SetNumUninitialized(CharacterNum);
		CharacterRangeTimes.SetNumUninitialized(CharacterNum);
		CharacterRangeStarts.SetNumUninitialized(CharacterNum);
		CharacterRangeLengths.SetNumUninitialized(CharacterNum);
		CharacterControlSetIndices.SetNumUninitialized(CharacterNum);
		CharacterControlSetEntries.SetNumUninitialized(CharacterNum);
		CharacterControlSetRanges.SetNumUninitialized(CharacterNum);
		CharacterControlSetFrames.SetNumUninitialized(CharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			int32 ControlSetIdx = INDEX_NONE, RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
			const bool bFound = Controller::Private::FindControlSetsTotalRange(ControlSetIdx, RangeEntry, RangeRange, Controller->TrainingSettings->ControlSets, ActiveRanges[CharacterIdx]);
			check(bFound);

			const UE::Learning::FFrameRangeSet& FrameRangeSet = Controller->TrainingSettings->ControlSets[ControlSetIdx].Data->FrameRangeSet;

			CharacterSequenceIndices[CharacterIdx] = FrameRangeSet.GetEntrySequence(RangeEntry);
			CharacterRangeStarts[CharacterIdx] = FrameRangeSet.GetEntryRangeStart(RangeEntry, RangeRange);
			CharacterRangeLengths[CharacterIdx] = FrameRangeSet.GetEntryRangeLength(RangeEntry, RangeRange);

			CharacterRangeTimes[CharacterIdx] = 0.0;
			if (ActiveRanges.Num() == 1)
			{
				CharacterRangeTimes[CharacterIdx] = EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() / TrainingDatabase->GetFrameRate().AsDecimal();
			}
			else
			{
				CharacterRangeTimes[CharacterIdx] = FMath::Clamp(
					EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() + (float)CharacterRangeStarts[CharacterIdx],
					(float)CharacterRangeStarts[CharacterIdx],
					(float)(CharacterRangeStarts[CharacterIdx] + CharacterRangeLengths[CharacterIdx] - 1)) / TrainingDatabase->GetFrameRate().AsDecimal();
			}

			CharacterRangeTimes[CharacterIdx] = FMath::Clamp(CharacterRangeTimes[CharacterIdx],
				0.0f, FMath::Max(TrainingDatabase->GetSequenceDuration(CharacterSequenceIndices[CharacterIdx]), 0.0f));

			// Find current control set and nearest frame in that control set
			CharacterControlSetIndices[CharacterIdx] = ControlSetIdx;
			CharacterControlSetEntries[CharacterIdx] = RangeEntry;
			CharacterControlSetRanges[CharacterIdx] = RangeRange;
			CharacterControlSetFrames[CharacterIdx] = FMath::Clamp(
				FMath::RoundToInt(CharacterRangeTimes[CharacterIdx] * TrainingDatabase->GetFrameRate().AsDecimal()) - CharacterRangeStarts[CharacterIdx],
				0, CharacterRangeLengths[CharacterIdx] - 1);

			check(TrainingDatabase->GetAnimSequence(CharacterSequenceIndices[CharacterIdx]))
		}

		// Resize Pose State and re-create Network Inference Instances

		const int32 BoneNum = TrainingDatabase->GetBoneNum();

		if (!PoseState.IsValid())
		{
			PoseState = UAnimDatabasePoseStateLibrary::MakePoseState();
		}

		TArray<int32> BoneParents;
		TrainingDatabase->GetBoneParents(BoneParents);

		PoseState.CurrPoseData->Resize(CharacterNum, BoneNum, TrainingAutoEncoder->GetAttributeTypes(), TrainingAutoEncoder->GetAttributeNames());
		PoseState.PoseGlobalBoneData->Resize(CharacterNum, BoneNum);

		// Frame Attributes

		const int32 DatabaseContentHash = TrainingDatabase->GetContentHash();

		if (FrameAttributeEntries != TrainingAutoEncoder->TrainedFrameAttributes ||
			FrameAttributesDatabaseContentHash != DatabaseContentHash)
		{
			FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingFrameAttributes", "Rebuilding AutoEncoder Frame Attributes in Database"));
			SlowTask.MakeDialog();

			FrameAttributeEntries = TrainingAutoEncoder->TrainedFrameAttributes;
			FrameAttributesDatabaseContentHash = DatabaseContentHash;

			const int32 AttributeNum = FrameAttributeEntries.Num();
			FrameAttributes.SetNum(AttributeNum);
			for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
			{
				FrameAttributes[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(
					TrainingDatabase,
					Controller->TrainingSettings->QueryRanges,
					FrameAttributeEntries[AttributeIdx].FrameAttribute);

				if (!FrameAttributes[AttributeIdx].IsValid())
				{
					FrameAttributes[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeNullFrameAttribute(Controller->TrainingSettings->QueryRanges);
				}
			}
		}

		// Evaluate Pose and attributes

		TrainingDatabase->WaitForCompressionOnAnimSequences(CharacterSequenceIndices);

		TrainingDatabase->SamplePoseData(
			PoseState.CurrPoseData->View(),
			CharacterSequenceIndices,
			CharacterRangeTimes,
			FrameAttributes);

		if (Controller->ViewportSettings->bRangesStartAtOrigin)
		{
			for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
			{
				const FVector RootOffset = Controller::Private::GetRangeOffset(
					CharacterIdx, CharacterNum,
					Controller->ViewportSettings->RangeOffsetSpacing,
					Controller->ViewportSettings->bOffsetRanges,
					Controller->ViewportSettings->bOffsetRangesInGrid);

				FTransform RangeStartRootTransform;
				TrainingDatabase->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1),
					CharacterSequenceIndices[CharacterIdx], CharacterRangeStarts[CharacterIdx]);

				Controller::Private::AdjustTransformRootTransform(
					PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootLinearVelocities[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootAngularVelocities[CharacterIdx],
					RootOffset,
					RangeStartRootTransform,
					Controller->ViewportSettings->bDisableRootTranslation,
					Controller->ViewportSettings->bDisableRootRotation);
			}
		}

		// Compute Forward Kinematics

		AnimDatabase::PoseData::ForwardKinematics(
			PoseState.PoseGlobalBoneData->View(),
			PoseState.CurrPoseData->LocalBoneData.ConstView(),
			PoseState.CurrPoseData->RootData.ConstView(),
			BoneParents);

		// Update Pose

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			if (PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset())
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FTransform BoneTransform = FTransform(
						((FQuat)PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx][BoneIdx]).GetNormalized(),
						(FVector)PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx],
						(FVector)PoseState.PoseGlobalBoneData->BoneScales[CharacterIdx][BoneIdx]);

					PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetBoneTransformByName(TrainingDatabase->GetBoneName(BoneIdx), BoneTransform, EBoneSpaces::WorldSpace);
				}

				PreviewScene->GetPoseableMeshComponent(CharacterIdx)->RefreshBoneTransforms();
			}
		}

		// Update UI Control Objects

		if (Controller->TrainingSettings->ControlObject.ObservationObject.IsValid() && CharacterNum > 0)
		{
			TArray<TSharedPtr<FControlObjectTreeElement>>& ControlObjectTreeElements = EditorToolKit->ControlObjectWidget->GetTreeElementsRef();

			bool bRefreshControlUI = ControlObjectTreeElements.Num() != CharacterNum;

			ControlObjectTreeElements.SetNum(CharacterNum);

			for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
			{
				if (!ControlObjectTreeElements[CharacterIdx].IsValid())
				{
					ControlObjectTreeElements[CharacterIdx] = MakeShared<FControlObjectTreeElement>();
					bRefreshControlUI = true;
				}

				const int32 FlatIndex = Controller->TrainingSettings->ControlSets[CharacterControlSetIndices[CharacterIdx]].Data->FrameRangeSet.GetEntryRangeOffset(
					CharacterControlSetEntries[CharacterIdx], CharacterControlSetRanges[CharacterIdx]) + CharacterControlSetFrames[CharacterIdx];

				bRefreshControlUI |= Controller::Private::UpdateControlObjectTreeElement(
					ControlObjectTreeElements[CharacterIdx],
					Controller->TrainingSettings->ControlObject.ObservationObject->Object,
					Controller->TrainingSettings->ControlSets[CharacterControlSetIndices[CharacterIdx]].Data->Controls[FlatIndex].ObjectElement);

				ControlObjectTreeElements[CharacterIdx]->Color = Controller->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
			}

			if (bRefreshControlUI)
			{
				EditorToolKit->ControlObjectWidget->RefreshTreeElements();
			}
		}
		else
		{
			TArray<TSharedPtr<FControlObjectTreeElement>>& ControlObjectTreeElements = EditorToolKit->ControlObjectWidget->GetTreeElementsRef();
			if (!ControlObjectTreeElements.IsEmpty())
			{
				ControlObjectTreeElements.Empty();
				EditorToolKit->ControlObjectWidget->RefreshTreeElements();
			}
		}

		// Update test range time if playing animation

		EditorToolKit->TimelineModel->UpdateTimelineFrameTime(DeltaTime * TrainingDatabase->GetFrameRate().AsDecimal());
	}

	void FControllerMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		if (TSharedPtr<FControllerToolkit> ToolKit = WeakToolkit.Pin())
		{
			// Load Database

			UAnimDatabase* TrainingDatabase = ToolKit->Controller->TrainingSettings->Database.Get();
			if (!TrainingDatabase) { return; }

			// Load AutoEncoder

			UAnimGenAutoEncoder* TrainingAutoEncoder = ToolKit->Controller->TrainingSettings->AutoEncoder.Get();
			if (!TrainingAutoEncoder || !TrainingAutoEncoder->IsValid()) { return; }

			// Load Behavior

			TObjectPtr<UAnimGenBehavior> TrainingBehavior = ToolKit->Controller->TrainingSettings->Behavior;
			if (!TrainingBehavior) { return; }

			// Draw Skeleton

			if (ToolKit->Controller->ViewportSettings->bDrawSkeleton)
			{
				// Find bone indices to render

				const int32 BoneNum = TrainingAutoEncoder->GetBoneNum();

				TArray<int32> BoneIndices;

				if (ToolKit->Controller->ViewportSettings->bDrawRequiredBonesOnly)
				{
					BoneIndices = TrainingAutoEncoder->AutoEncodedRequiredBoneIndices;
				}
				else
				{
					BoneIndices.SetNumUninitialized(BoneNum);
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						BoneIndices[BoneIdx] = BoneIdx;
					}
				}

				const FLinearColor DefaultBoneColor = GetDefault<UPersonaOptions>()->DefaultBoneColor;

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					// Draw Skeleton

					FLinearColor BoneColor = DefaultBoneColor;
					BoneColor.A = ToolKit->Controller->ViewportSettings->DrawSkeletonOpacity;

					FDrawDebugSkeletonSettings SkeletonSettings;
					SkeletonSettings.bDrawSimpleSkeleton = ToolKit->Controller->ViewportSettings->bDrawSimpleSkeleton;
					SkeletonSettings.BoneRadius = ToolKit->Controller->ViewportSettings->DrawSkeletonScale;

					UDrawDebugLibrary::DrawDebugSkeletonArrayView(
						FDebugDrawer::MakeDebugDrawer(PDI),
						PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
						PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
						PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
						PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
						BoneIndices,
						TrainingAutoEncoder->GetBoneParents(),
						BoneColor,
						false,
						SkeletonSettings);

					// Draw Velocities

					FLinearColor VelocityBoneColor = BoneColor;
					VelocityBoneColor.A = ToolKit->Controller->ViewportSettings->DrawLinearVelocitiesOpacity;

					if (ToolKit->Controller->ViewportSettings->bDrawLinearVelocities)
					{
						UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneLinearVelocities[CharacterIdx],
							BoneIndices,
							VelocityBoneColor,
							ToolKit->Controller->ViewportSettings->DrawSkeletonScale * 0.5f,
							true,
							ToolKit->Controller->ViewportSettings->DrawLinearVelocitiesScale);
					}

					if (ToolKit->Controller->ViewportSettings->bDrawAngularVelocities)
					{
						UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneAngularVelocities[CharacterIdx],
							BoneIndices,
							VelocityBoneColor,
							ToolKit->Controller->ViewportSettings->DrawSkeletonScale * 0.5f,
							true,
							ToolKit->Controller->ViewportSettings->DrawAngularVelocitiesScale);
					}

					// Draw Transforms

					if (ToolKit->Controller->ViewportSettings->bDrawBoneTransforms)
					{
						UDrawDebugLibrary::DrawDebugTransformsArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneScales[CharacterIdx],
							UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(BoneColor),
							false,
							ToolKit->Controller->ViewportSettings->DrawSkeletonScale * 5.0f);
					}
				}
			}

			// Draw Range Identifiers

			if (ToolKit->Controller->ViewportSettings->bDrawRangeIdentifier)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->Controller->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Controller->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Controller->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->Controller->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector IdentifierLocation = (FVector)PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
						(FVector)PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector(FVector3f(0.0f, 0.0f, ToolKit->Controller->ViewportSettings->DrawRangeIdentifierHeight));

					FDrawDebugLineStyle LineStyle;
					LineStyle.Color = IdentifierColor;
					LineStyle.Thickness = ToolKit->Controller->ViewportSettings->DrawRangeIdentifierThickness;

					UDrawDebugLibrary::DrawDebugSimpleSphere(
						FDebugDrawer::MakeDebugDrawer(PDI),
						IdentifierLocation,
						FRotator::ZeroRotator,
						LineStyle,
						true,
						ToolKit->Controller->ViewportSettings->DrawRangeIdentifierRadius);
				}
			}

			// Draw Skeleton Roots

			if (ToolKit->Controller->ViewportSettings->bDrawRoot && CharacterNum > 0)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->Controller->ViewportSettings->DrawRootOpacity;
				LineStyle.Thickness = ToolKit->Controller->ViewportSettings->DrawSkeletonScale;

				UDrawDebugLibrary::DrawDebugRotationsQuatArrayView(
					FDebugDrawer::MakeDebugDrawer(PDI),
					PoseState.CurrPoseData->RootData.RootLocations,
					PoseState.CurrPoseData->RootData.RootRotations,
					LineStyle,
					true,
					ToolKit->Controller->ViewportSettings->DrawRootScale);
			}

			// Draw Origin

			if (ToolKit->Controller->ViewportSettings->bDrawOrigin)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->Controller->ViewportSettings->DrawOriginOpacity;
				LineStyle.Thickness = ToolKit->Controller->ViewportSettings->DrawOriginLineThickness;

				UDrawDebugLibrary::DrawDebugTransform(
					FDebugDrawer::MakeDebugDrawer(PDI),
					FTransform::Identity,
					LineStyle,
					true,
					ToolKit->Controller->ViewportSettings->DrawOriginScale);
			}

			// Draw Trajectory

			if (ToolKit->Controller->ViewportSettings->bDrawTrajectories &&
				(!ToolKit->Controller->ViewportSettings->bRangesStartAtOrigin ||
					!ToolKit->Controller->ViewportSettings->bDisableRootTranslation))
			{
				FVector ForwardVector = FVector::ForwardVector;
				switch (TrainingAutoEncoder->TrainedSkeleton->GetPreviewForwardAxis())
				{
				case EAxis::X: ForwardVector = FVector::XAxisVector; break;
				case EAxis::Y: ForwardVector = FVector::YAxisVector; break;
				case EAxis::Z: ForwardVector = FVector::ZAxisVector; break;
				}

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					const FVector RootOffset = Controller::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->Controller->ViewportSettings->RangeOffsetSpacing,
						ToolKit->Controller->ViewportSettings->bOffsetRanges,
						ToolKit->Controller->ViewportSettings->bOffsetRangesInGrid);

					UDrawDebugLibrary::DrawDebugRangeTrajectoryArrayView(
						FDebugDrawer::MakeDebugDrawer(PDI),
						RangeTrajectoryLocations[ActiveRanges[CharacterIdx]],
						RangeTrajectoryRotations[ActiveRanges[CharacterIdx]],
						true,
						ForwardVector,
						ToolKit->Controller->ViewportSettings->bDrawTrajectoryOrientations,
						ToolKit->Controller->ViewportSettings->bRangesStartAtOrigin,
						RootOffset);
				}
			}

			// Debug Draw Controls

			if (ToolKit->Controller->ViewportSettings->bDrawDebugTraining)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->Controller->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Controller->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Controller->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->Controller->ViewportSettings->DrawRangeIdentifierOpacity;

					const int32 FlatIndex = ToolKit->Controller->TrainingSettings->ControlSets[CharacterControlSetIndices[CharacterIdx]].Data->FrameRangeSet.GetEntryRangeOffset(
						CharacterControlSetEntries[CharacterIdx], CharacterControlSetRanges[CharacterIdx]) + CharacterControlSetFrames[CharacterIdx];

					PoseState.PoseIdx = CharacterIdx;

					TrainingBehavior->DrawDebugControl(
						FDebugDrawer::MakeDebugDrawer(PDI),
						FDebugDrawer::MakeDebugDrawer(static_cast<AnimDatabase::Editor::FViewportClient*>(Viewport->GetClient())->GetOrMakeDebugDrawBuffer()),
						ToolKit->Controller->TrainingSettings->ControlObject,
						ToolKit->Controller->TrainingSettings->ControlSets[CharacterControlSetIndices[CharacterIdx]].Data->Controls[FlatIndex],
						PoseState,
						TrainingDatabase,
						TrainingAutoEncoder,
						CharacterIdx,
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeTimes[CharacterIdx],
						CharacterRangeStarts[CharacterIdx],
						CharacterRangeLengths[CharacterIdx],
						IdentifierColor);
				}
			}

			// Debug Draw

			if (ToolKit->Controller->TrainingSettings->DebugDrawer)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->Controller->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Controller->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Controller->TrainingSettings->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->Controller->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector RootOffset = Controller::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->Controller->ViewportSettings->RangeOffsetSpacing,
						ToolKit->Controller->ViewportSettings->bOffsetRanges,
						ToolKit->Controller->ViewportSettings->bOffsetRangesInGrid);

					FTransform RangeStartRootTransform = FTransform::Identity;
					TrainingDatabase->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1),
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeStarts[CharacterIdx]);

					const FTransform RangeViewportTransform = !ToolKit->Controller->ViewportSettings->bRangesStartAtOrigin ? FTransform::Identity
						: RangeStartRootTransform.Inverse() * FTransform(FQuat::Identity, RootOffset, FVector::OneVector);

					PoseState.PoseIdx = CharacterIdx;

					ToolKit->Controller->TrainingSettings->DebugDrawer->DrawDebug(
						FDebugDrawer::MakeDebugDrawer(PDI),
						FDebugDrawer::MakeDebugDrawer(static_cast<AnimDatabase::Editor::FViewportClient*>(Viewport->GetClient())->GetOrMakeDebugDrawBuffer()),
						PoseState,
						TrainingDatabase,
						UAnimDatabaseFrameRangesLibrary::FrameRangesRangeAtIndex(ToolKit->Controller->TrainingSettings->QueryRanges, ActiveRanges[CharacterIdx]),
						RangeViewportTransform,
						CharacterIdx,
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeTimes[CharacterIdx],
						CharacterRangeStarts[CharacterIdx],
						CharacterRangeLengths[CharacterIdx],
						IdentifierColor);
				}
			}
		}

		FEdMode::Render(View, Viewport, PDI);
	}
}

#undef LOCTEXT_NAMESPACE