// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditorControllerToolkit.h"

#include "AnimGenEditorControllerMode.h"
#include "AnimGenEditorTraining.h"
#include "SAnimGenEditorTraining.h"
#include "SAnimGenEditorControlObject.h"

#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorViewportClient.h"
#include "AnimDatabaseEditorTimeline.h"
#include "SAnimDatabaseEditorTimeline.h"

#include "AnimGenLog.h"
#include "AnimGenControl.h"
#include "AnimGenController.h"
#include "AnimGenAutoEncoder.h"
#include "AnimGenBehavior.h"

#include "AnimDatabase.h"
#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameAttribute.h"

#include "LearningTrainer.h"
#include "LearningNeuralNetwork.h"
#include "LearningRandom.h"
#include "LearningArray.h"
#include "LearningSharedMemory.h"

#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "AdvancedPreviewSceneModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Application/SlateApplication.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"
#include "Viewports.h"
#include "PreviewProfileController.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SScrollBox.h"
#include "AssetTypeCategories.h"
#include "PipInstallHelpers.h"

#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"

#include "Misc/MonitoredProcess.h"
#include "Misc/Guid.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "Async/ParallelFor.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "NNERuntimeBasicCpuBuilder.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorControllerToolkit"


FText UAnimGenEditorControllerAssetDefinition::GetAssetDisplayName() const
{
	return LOCTEXT("AnimGenEditorControllerAssetDefinitionName", "Controller");
}

FLinearColor UAnimGenEditorControllerAssetDefinition::GetAssetColor() const
{
	return FColor(181, 177, 165);
}

TSoftClassPtr<UObject> UAnimGenEditorControllerAssetDefinition::GetAssetClass() const
{
	return UAnimGenController::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAnimGenEditorControllerAssetDefinition::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("AnimGenControllerAssetDefinitionMenu", "AnimGen")) };
	return Categories;
}

EAssetCommandResult UAnimGenEditorControllerAssetDefinition::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FPythonScriptInitHelper::InitPythonAndPipInstall(FSimpleDelegate::CreateLambda([
		OpenArgs = FAssetOpenArgs(OpenArgs),
		// Taking a copy of OpenArgs.Assets as it's a TArrayView and this callback outlives the view
		OpenAssetsData = TArray<FAssetData>(OpenArgs.Assets)]()
		{
			FAssetArgs WrappedArgs(OpenAssetsData);
			for (UAnimGenController* Asset : WrappedArgs.LoadObjects<UAnimGenController>())
			{
				TSharedRef<UE::AnimGen::Editor::FControllerToolkit> NewEditor(new UE::AnimGen::Editor::FControllerToolkit());
				NewEditor->InitAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Asset);
			}
		}),
		FSimpleDelegate::CreateLambda([]()
			{
				UE_LOGF(LogAnimGen, Warning, "AnimGen Controller toolkit may not function properly without Python enabled");
			}));

	return EAssetCommandResult::Handled;
}

UAnimGenControllerEditorFactory::UAnimGenControllerEditorFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAnimGenController::StaticClass();
}

UObject* UAnimGenControllerEditorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimGenController>(InParent, Name, Flags | RF_Transactional);
}

bool UAnimGenControllerEditorFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UAnimGenControllerEditorFactory::ConfigureProperties()
{
	return true;
}

FText UAnimGenControllerEditorFactory::GetDisplayName() const
{
	return LOCTEXT("AnimGenControllerEditorAsset_DisplayName", "Controller");
}

uint32 UAnimGenControllerEditorFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UAnimGenControllerEditorFactory::GetToolTip() const
{
	return LOCTEXT("AnimGenAControllerEditorAsset_Tooltip", "Asset for training a model which can create animation following a provided behavior.");
}

FString UAnimGenControllerEditorFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("AGC_NewAnimGenController"));
}

const TArray<FText>& UAnimGenControllerEditorFactory::GetMenuCategorySubMenus() const
{
	static TArray<FText> SubMenus{ LOCTEXT("SubMenuAnimGen", "AnimGen") };
	return SubMenus;
}

namespace UE::AnimGen::Editor
{
	TSharedRef<IDetailCustomization> FControllerTrainingSettingsDetails::MakeInstance()
	{
		return MakeShared<FControllerTrainingSettingsDetails>();
	}

	void FControllerTrainingSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Get the UAnimGenControllerTrainingSettings object we are associated with

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1) { return; }

		UAnimGenControllerTrainingSettings* TrainingSettings = StaticCast<UAnimGenControllerTrainingSettings*>(Objects[0].Pin().Get());
		if (!TrainingSettings) { return; }

		// Reset the selected items since we are re-drawing the UI

		SelectedItems.Reset();

		// Data Category

		IDetailCategoryBuilder& DataCategory = DetailBuilder.EditCategory("Data", LOCTEXT("DataCategory", "Data"), ECategoryPriority::Important);
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, Database), UAnimGenControllerTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, FrameRanges), UAnimGenControllerTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, AutoEncoder), UAnimGenControllerTrainingSettings::StaticClass());
		DataCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenControllerTrainingSettings, Behavior), UAnimGenControllerTrainingSettings::StaticClass());

		if (TrainingSettings->Database.Get() &&
			TrainingSettings->Behavior && (
			!TrainingSettings->ControlSchema.IsValid() || 
			!TrainingSettings->ControlSchema.IsElementValid(TrainingSettings->ControlSchemaElement)))
		{
			FDetailWidgetRow& NeedsReimportErrorRow = DataCategory.AddCustomRow(LOCTEXT("InvalidBehaviorWarning", "Error"))
				.WholeRowContent()
				[
					SNew(SBox)
						.Padding(FMargin(0.0f, 4.0f))
						[
							SNew(SWarningOrErrorBox)
								.MessageStyle(EMessageStyle::Error)
								.Message(LOCTEXT("InvalidBehaviorErrorMessage",
									"Behavior is invalid."))
						]
				];
		}

		IDetailCategoryBuilder& ControlSetsCategory = DetailBuilder.EditCategory("ControlSets", LOCTEXT("ControlSetsCategory", "Control Sets"), ECategoryPriority::Important);

		// Add the ListView 

		if (TrainingSettings->QueryEntries.Num() > 0)
		{
			// Make the ListView used to render the FQueryEntry array

			SAssignNew(ListView, SListView<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>)
				.ListItemsSource(&TrainingSettings->QueryEntries)
				.OnGenerateRow(this, &FControllerTrainingSettingsDetails::OnGenerateRow)
				.OnSelectionChanged(this, &FControllerTrainingSettingsDetails::OnSelectionChanged)
				.HeaderRow(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Icon")
					.DefaultLabel(FText::GetEmpty())
					.FillWidth(0.05f)
					+ SHeaderRow::Column("Sequence")
					.DefaultLabel(LOCTEXT("ColumnLabelAnimSequence", "Anim Sequence"))
					.FillWidth(0.7f)
					+ SHeaderRow::Column("Start")
					.DefaultLabel(LOCTEXT("ColumnLabelStart", "Start"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Stop")
					.DefaultLabel(LOCTEXT("ColumnLabelStop", "Stop"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Mirrored")
					.DefaultLabel(LOCTEXT("ColumnLabelMirrored", "Mirrored"))
					.FillWidth(0.05f)
				);

			// Select first entry if we are re-drawing the UI

			ListView->SetSelection(TrainingSettings->QueryEntries[0]);

			// Add to UI

			ControlSetsCategory.AddCustomRow(LOCTEXT("CategoryRanges", "Ranges"))
				.WholeRowContent()
				[
					SNew(SBox)
					// We need to set the list view to a fixed maximum height otherwise it will fill the whole details panel. Unfortunately the 
					// scroll bar of the details panel will get messed up if we make this too large and this is a wont-fix right now (UE-40387)
					// we therefore try to limit the size a bit even if this is annoying for users.
					.MaxDesiredHeight(500.0f)
					[
						ListView.ToSharedRef()
					]
				];

			// Re-set selection

			TArray<TSharedPtr<AnimDatabase::Editor::FQueryEntry>, TInlineAllocator<16>> ResetSelectedItems;
			for (const TSharedPtr<AnimDatabase::Editor::FQueryEntry>& Item : ListView->GetItems())
			{
				if (Item->bIsSelected)
				{
					ResetSelectedItems.Add(Item);
				}
			}
			ListView->SetItemSelection(ResetSelectedItems, true);
		}
		else
		{
			// Write out message instead of query returned no ranges

			ControlSetsCategory.AddCustomRow(LOCTEXT("CategoryRanges", "Ranges"))
				.WholeRowContent()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(
						TrainingSettings->Database.IsNull() ? LOCTEXT("NoDatabase", "No Training Database Provided") :
						!TrainingSettings->Database.Get() ? LOCTEXT("LoadingDatabase", "Loading Training Database...") :
						!TrainingSettings->AutoEncoder.Get() ? LOCTEXT("LoadingAutoEncoder", "Loading Training AutoEncoder...") :
						LOCTEXT("EmptyRanges", "No Control Sets returned by Behavior"))
				];
		}
	}

	void FControllerTrainingSettingsDetails::OnSelectionChanged(TSharedPtr<AnimDatabase::Editor::FQueryEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		ListView->GetSelectedItems(SelectedItems);
		for (const TSharedPtr<AnimDatabase::Editor::FQueryEntry>& Item : ListView->GetItems())
		{
			Item->bIsSelected = SelectedItems.Contains(Item);
		}
	}

	TSharedRef<ITableRow> FControllerTrainingSettingsDetails::OnGenerateRow(TSharedPtr<AnimDatabase::Editor::FQueryEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		class SFilterMultiColumnRow : public SMultiColumnTableRow<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>
		{
		public:
			SLATE_BEGIN_ARGS(SFilterMultiColumnRow) {}
				SLATE_ARGUMENT(TSharedPtr<AnimDatabase::Editor::FQueryEntry>, Item)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
			{
				Item = InArgs._Item;
				SMultiColumnTableRow<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
			}

			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
			{
				if (ColumnName == "Icon")
				{
					// Draw colored circle icon a.k.a "Range Identifier"
					return SNew(SBox)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.MaxDesiredWidth(10.0f)
						.MaxDesiredHeight(10.0f)
						.MaxAspectRatio(1.0f)
						.MinAspectRatio(1.0f)
						[
							SNew(SImage)
								.ColorAndOpacity(Item->Color)
								.Image(FCoreStyle::Get().GetBrush("GraphEditor.PinIcon"))
						];
				}
				else if (ColumnName == "Sequence")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SHyperlink).Text(FText::FromString(Item->Sequence->GetName())).OnNavigate_Lambda([this]()
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Item->Sequence);
								})
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(Item->bIsMirrored ? LOCTEXT("MirroredAddition", " (mirrored)") : FText::GetEmpty())
						];
				}
				else if (ColumnName == "Start")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StartFrame))
						];
				}
				else if (ColumnName == "Stop")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StopFrame))
						];
				}
				else if (ColumnName == "Mirrored")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SImage).Image(FCoreStyle::Get().GetBrush("GraphEditor.AlignNodesCenter")).RenderOpacity(Item->bIsMirrored ? 1.0f : 0.0f)
						];
				}

				return SNullWidget::NullWidget;
			}

		private:
			TSharedPtr<AnimDatabase::Editor::FQueryEntry> Item;
		};

		return SNew(SFilterMultiColumnRow, OwnerTable).Item(Item);
	}


	TSharedRef<IDetailCustomization> FControllerDetails::MakeInstance()
	{
		return MakeShared<FControllerDetails>();
	}

	void FControllerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Get the UAnimGenController object we are associated with

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1) { return; }

		UAnimGenController* Controller = StaticCast<UAnimGenController*>(Objects[0].Pin().Get());
		if (!Controller) { return; }

		// Add Categories

		IDetailCategoryBuilder& ControllerCategory = DetailBuilder.EditCategory("Controller", LOCTEXT("ControllerCategory", "Controller"), ECategoryPriority::Important);
		ControllerCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenController, AutoEncoder), UAnimGenController::StaticClass());
		ControllerCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenController, FrameRate), UAnimGenController::StaticClass());

		// Check if Trained Database has been modified

		if (Controller->IsValid() && Controller->TrainingSettings->Database.Get())
		{
			const UAnimDatabase* TrainingDatabase = Controller->TrainingSettings->Database.Get();

			const FAnimDatabaseFrameRanges TrainingFrameRanges = Controller->TrainingSettings->FrameRanges ?
				UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(TrainingDatabase, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase), Controller->TrainingSettings->FrameRanges) :
				UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase);

			if (UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(TrainingDatabase, TrainingFrameRanges) != Controller->TrainedFrameRangesContentHash)
			{
				FDetailWidgetRow& NeedsReimportErrorRow = ControllerCategory.AddCustomRow(LOCTEXT("ModifiedWarning", "Warning"))
					.WholeRowContent()
					[
						SNew(SBox)
							.Padding(FMargin(0.0f, 4.0f))
							[
								SNew(SWarningOrErrorBox)
									.MessageStyle(EMessageStyle::Warning)
									.Message(LOCTEXT("DatabaseModifiedWarningMessage",
										"Training Data appears to have changed since Controller was last trained."))
							]
					];
			}
		}

		if (Controller->IsValid() && Controller->AutoEncoder && Controller->AutoEncoder->GetContentHash() != Controller->TrainedAutoEncoderContentHash)
		{
			FDetailWidgetRow& NeedsReimportErrorRow = ControllerCategory.AddCustomRow(LOCTEXT("ModifiedWarning", "Warning"))
				.WholeRowContent()
				[
					SNew(SBox)
						.Padding(FMargin(0.0f, 4.0f))
						[
							SNew(SWarningOrErrorBox)
								.MessageStyle(EMessageStyle::Error)
								.Message(LOCTEXT("AutoEncoderModifiedWarningMessage",
									"Auto-Encoder appears to have changed since Controller was last trained. Controller may no longer function correctly."))
						]
				];
		}

		if (Controller->IsValid() && Controller->TrainingSettings->Behavior)
		{
			FAnimGenControlSchema ControllerSchema = UAnimGenControls::MakeControlSchema();
			const FAnimGenControlSchemaElement SchemaElement = Controller->TrainingSettings->Behavior->SpecifyControl(ControllerSchema);

			if (Learning::Observation::GetSchemaObjectsCompatibilityHash(ControllerSchema.ObservationSchema->Schema, SchemaElement.SchemaElement) != Controller->TrainedSchemaCompatibilityHash)
			{
				FDetailWidgetRow& NeedsReimportErrorRow = ControllerCategory.AddCustomRow(LOCTEXT("ModifiedWarning", "Warning"))
					.WholeRowContent()
					[
						SNew(SBox)
							.Padding(FMargin(0.0f, 4.0f))
							[
								SNew(SWarningOrErrorBox)
									.MessageStyle(EMessageStyle::Error)
									.Message(LOCTEXT("BehaviorModifiedWarningMessage",
										"Behavior appears to have changed since Controller was last trained."))
							]
					];
			}
		}

		IDetailCategoryBuilder& StatisticsCategory = DetailBuilder.EditCategory("Statistics", LOCTEXT("Statistics", "Statistics"), ECategoryPriority::Important);
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenController, ControlEncoderSize), UAnimGenController::StaticClass());
		StatisticsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGenController, DenoiserSize), UAnimGenController::StaticClass());
	}

	class SControllerViewportToolBar;

	/** Custom Controller Viewport class. Contains a custom show menu at the top which contains some toggles for the ViewportSettings */
	class SControllerViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
	{
	public:

		SLATE_BEGIN_ARGS(SControllerViewport) {}
		SLATE_END_ARGS();

		void Construct(
			const FArguments& InArgs,
			const TSharedRef<FControllerToolkit>& InAssetEditorToolkit,
			const TSharedRef<AnimDatabase::Editor::FPreviewScene>& InPreviewScene,
			const FEditorModeID InModeID);

		// ~ICommonEditorViewportToolbarInfoProvider interface
		virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
		virtual TSharedPtr<FExtender> GetExtenders() const override;
		virtual void OnFloatingButtonClicked() override {}
		// ~End of ICommonEditorViewportToolbarInfoProvider interface

	protected:

		// ~SEditorViewport interface
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
		// ~End of SEditorViewport interface

		/** Viewport client */
		TSharedPtr<AnimDatabase::Editor::FViewportClient> ViewportClient;

		/** The preview scene that we are viewing */
		TWeakPtr<AnimDatabase::Editor::FPreviewScene> PreviewScenePtr;

		/** Controller Toolkit */
		TWeakPtr<FControllerToolkit> ControllerToolkit;

		/** Editor Mode */
		FEditorModeID ModeID;
	};

	/** Custom viewport toolbar which refers to some settings in database ViewportSettings */
	class SControllerViewportToolBar : public SCommonEditorViewportToolbarBase
	{
	public:
		SLATE_BEGIN_ARGS(SControllerViewportToolBar) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<SControllerViewport> InViewport, TWeakPtr<FControllerToolkit> InControllerToolkit);
		virtual TSharedRef<SWidget> GenerateShowMenu() const override;

		TWeakPtr<FControllerToolkit> ControllerToolkit;
	};

	void SControllerViewport::Construct(
		const FArguments& InArgs,
		const TSharedRef<FControllerToolkit>& InAssetEditorToolkit,
		const TSharedRef<AnimDatabase::Editor::FPreviewScene>& InPreviewScene,
		const FEditorModeID InModeID)
	{
		PreviewScenePtr = InPreviewScene;
		ControllerToolkit = InAssetEditorToolkit.ToWeakPtr();
		ModeID = InModeID;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
		);
	}

	TSharedRef<SEditorViewport> SControllerViewport::GetViewportWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<FExtender> SControllerViewport::GetExtenders() const
	{
		TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
		return Result;
	}

	TSharedRef<FEditorViewportClient> SControllerViewport::MakeEditorViewportClient()
	{
		ViewportClient = MakeShared<AnimDatabase::Editor::FViewportClient>(
			PreviewScenePtr,
			SharedThis(this),
			StaticCastWeakPtr<FAssetEditorToolkit>(ControllerToolkit),
			ModeID);

		ViewportClient->ViewportType = LVT_Perspective;
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
		ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

		return ViewportClient.ToSharedRef();
	}

	TSharedPtr<SWidget> SControllerViewport::BuildViewportToolbar()
	{
		return SNew(SControllerViewportToolBar, SharedThis(this), ControllerToolkit);
	}

	void SControllerViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<SControllerViewport> InViewport, TWeakPtr<FControllerToolkit> InControllerToolkit)
	{
		SCommonEditorViewportToolbarBase::Construct(
			SCommonEditorViewportToolbarBase::FArguments()
			.AddRealtimeButton(false)
			.PreviewProfileController(MakeShared<FPreviewProfileController>()),
			InViewport);

		ControllerToolkit = InControllerToolkit;
	}

	TSharedRef<SWidget> SControllerViewportToolBar::GenerateShowMenu() const
	{
		GetInfoProvider().OnFloatingButtonClicked();

		TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
		{
			//-------------------------------

			ShowMenuBuilder.BeginSection("Skeleton", LOCTEXT("ShowMenu_SkeletonLabel", "Skeleton"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelSkeleton", "Skeleton"),
				LOCTEXT("ShowMenu_SkeletonLabelSkeletonToolTip", "Draw the character skeleton"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (ControllerToolkit.IsValid()) {
				ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawSkeleton = !ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawSkeleton;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return ControllerToolkit.IsValid() ? ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawSkeleton : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocities", "Linear Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocitiesToolTip", "Draw linear velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (ControllerToolkit.IsValid()) {
				ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawLinearVelocities = !ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawLinearVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return ControllerToolkit.IsValid() ? ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawLinearVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocities", "Angular Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocitiesToolTip", "Draw Angular velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (ControllerToolkit.IsValid()) {
				ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawAngularVelocities = !ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawAngularVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return ControllerToolkit.IsValid() ? ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawAngularVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Root", LOCTEXT("ShowMenu_RootLabel", "Root"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RootLabelRoot", "Root"),
				LOCTEXT("ShowMenu_RootLabelRootToolTip", "Draw Root"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (ControllerToolkit.IsValid()) {
				ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawRoot = !ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawRoot;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return ControllerToolkit.IsValid() ? ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawRoot : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Origin", LOCTEXT("ShowMenu_OriginLabel", "Origin"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_OriginLabelOrigin", "Origin"),
				LOCTEXT("ShowMenu_OriginLabelOriginToolTip", "Draw Origin"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (ControllerToolkit.IsValid()) {
				ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawOrigin = !ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawOrigin;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return ControllerToolkit.IsValid() ? ControllerToolkit.Pin()->Controller->ViewportSettings->bDrawOrigin : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------
		}

		return ShowMenuBuilder.MakeWidget();
	}

	struct FControllerTrainingModel : public ITrainingModel
	{
		FControllerTrainingModel() = default;
		FControllerTrainingModel(TWeakPtr<FControllerToolkit> InToolkit) : Toolkit(InToolkit) {}
		virtual ~FControllerTrainingModel() { if (IsTraining()) { StopTraining(); } }

		virtual bool IsEmpty() const override
		{
			if (TSharedPtr<FControllerToolkit> ToolkitPtr = Toolkit.Pin())
			{
				return !ToolkitPtr->Controller->IsValid();
			}

			return false;
		}

		virtual void StartTraining() override
		{
			if (IsTraining()) { return; }

			// The overall structure of this function is as follows:
			//
			// 1. First we need to extract all the data from our animation database and use the AutoEncoder to encode it so that we have a single vector per
			//    frame of animation in the database. We will also extract the sequence starts and stops so that we know where sequences begin and end in
			//    the database
			//
			// 2. Next we will compute the controls for each frame in the database. To do this we need to create the control schema using the blueprint 
			//    function we implemented. Then we can call the blueprint function MakeControlAtFrame for each frame in the database and convert the result
			//    to a flat vector we can store in a big array.
			//
			// 3. Then we need to build the neural network we are going to use. In this case I am building a recurrent neural network which takes as input
			//    the current control vector, the previous frame encoded pose vector, and the previous frame memory state, and outputs the current frame 
			//    encoded pose vector and the current frame memory state.
			//
			// 4. I also build another small neural network which takes as input an encoded pose vector, and outputs some initial memory state for the 
			//    recurrent network.
			// 
			// 5. Finally I write all the training configuration to a JSON file and launch the Python training subprocess.

			if (TSharedPtr<FControllerToolkit> ToolkitPtr = Toolkit.Pin())
			{
				// Clear Error Message

				ErrorMessage = FText();

				// Get Controller Asset

				UAnimGenController* Controller = ToolkitPtr->Controller;

				UAnimDatabase* TrainingDatabase = Controller->TrainingSettings->Database.LoadSynchronous();

				if (!TrainingDatabase)
				{
					ErrorMessage = LOCTEXT("DatabaseNotProvidedError", "Error: Database not Provided");
					return;
				}

				USkeleton* TrainingSkeleton = TrainingDatabase->GetSkeleton();

				if (!TrainingSkeleton)
				{
					ErrorMessage = LOCTEXT("SkeletonNotProvidedError", "Error: Database contains no Skeleton");
					return;
				}

				UAnimGenAutoEncoder* TrainingAutoEncoder = Controller->TrainingSettings->AutoEncoder.LoadSynchronous();

				if (!TrainingAutoEncoder)
				{
					ErrorMessage = LOCTEXT("AutoEncoderNotProvidedError", "Error: AutoEncoder not Provided");
					return;
				}

				if (!TrainingAutoEncoder->IsValid())
				{
					ErrorMessage = LOCTEXT("AutoEncoderNotTrainedError", "Error: AutoEncoder not Trained");
					return;
				}

				const UAnimGenBehavior* Behavior = Controller->TrainingSettings->Behavior;

				if (!Behavior)
				{
					ErrorMessage = LOCTEXT("BehaviorNotProvidedError", "Error: Behavior not Provided");
					return;
				}

				// Compute Training Frame Ranges

				const FAnimDatabaseFrameRanges TrainingFrameRanges = Controller->TrainingSettings->FrameRanges ?
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(TrainingDatabase, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase), Controller->TrainingSettings->FrameRanges) :
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(TrainingDatabase);

				const int32 TrainingRangesTotalRangeNum = TrainingFrameRanges.FrameRangeSet->GetTotalRangeNum();

				if (TrainingRangesTotalRangeNum == 0)
				{
					ErrorMessage = LOCTEXT("TrainingFrameEmptyError", "Error: Training Frame Ranges are empty");
					return;
				}

				// Checking Empty and Additives

				TArray<int32> TrainingRangeSequenceIndices;
				TrainingRangeSequenceIndices.SetNumUninitialized(TrainingRangesTotalRangeNum);
				Learning::FrameRangeSet::AllRangeSequences(TrainingRangeSequenceIndices, *TrainingFrameRanges.FrameRangeSet);

				for (int32 RangeIdx = 0; RangeIdx < TrainingRangesTotalRangeNum; RangeIdx++)
				{
					const UAnimSequence* AnimSequence = TrainingDatabase->GetAnimSequence(TrainingRangeSequenceIndices[RangeIdx]);

					if (!AnimSequence)
					{
						ErrorMessage = LOCTEXT("NullAnimationError", "Error: Training data contains null animations.");
						return;
					}

					if (AnimSequence->IsValidAdditive())
					{
						ErrorMessage = FText::Format(LOCTEXT("AdditiveAnimationError", "Error: Training data contains additive animations: {0}."), FText::FromString(AnimSequence->GetName()));
						return;
					}
				}

				// Make Schema and compute compatibility hash

				FAnimGenControlSchema ControllerSchema = UAnimGenControls::MakeControlSchema();
				const FAnimGenControlSchemaElement SchemaElement = Behavior->SpecifyControl(ControllerSchema);

				if (!ControllerSchema.ObservationSchema->Schema.IsValid(SchemaElement.SchemaElement))
				{
					ErrorMessage = LOCTEXT("InvalidSchemaError", "Error: Invalid Schema Element returned");
					return;
				}

				// Compute Control Sets

				FAnimGenControlObject ControlObject = UAnimGenControls::MakeControlObject();
				TArray<FAnimGenControlSet> ControlSets;
				Behavior->MakeControlSets(ControlSets, ControlObject, TrainingDatabase, TrainingFrameRanges);

				const int32 ControlSetNum = ControlSets.Num();

				if (ControlSets.IsEmpty())
				{
					ErrorMessage = LOCTEXT("NoControlSetsError", "Error: No Control Sets Provided for Training");
					return;
				}

				for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
				{
					if (!ControlSets[ControlSetIdx].IsValid())
					{
						ErrorMessage = LOCTEXT("InvalidControlSetsError", "Error: Invalid Control Sets Provided for Training");
						return;
					}

					const int32 ControlSetObjectNum = ControlSets[ControlSetIdx].GetControls().Num();

					for (int32 ControlSetObjectIdx = 0; ControlSetObjectIdx < ControlSetObjectNum; ControlSetObjectIdx++)
					{
						if (!UAnimGenControls::ValidateControlObjectMatchesSchema(
							ControllerSchema,
							SchemaElement,
							ControlObject,
							ControlSets[ControlSetIdx].GetControls()[ControlSetObjectIdx]))
						{
							ErrorMessage = LOCTEXT("InvalidControlSetObject", "Error: Control Set does not match format given by SpecifyControls (see log)");
							return;
						}
					}
				}

				// Allocate and add frame ranges required by all control sets

				FAnimDatabaseFrameRanges DatabaseFrameRanges = UAnimGenControls::ControlSetFrameRanges(ControlSets[0]);
				for (int32 ControlSetIdx = 1; ControlSetIdx < ControlSetNum; ControlSetIdx++)
				{
					DatabaseFrameRanges = UAnimDatabaseFrameRangesLibrary::FrameRangesUnion(DatabaseFrameRanges, UAnimGenControls::ControlSetFrameRanges(ControlSets[ControlSetIdx]));
				}

				const int32 DatabaseTotalRangeNum = DatabaseFrameRanges.FrameRangeSet->GetTotalRangeNum();

				if (DatabaseTotalRangeNum == 0)
				{
					ErrorMessage = LOCTEXT("ControlSetsNoRangesError", "Error: Control Sets contain no ranges.");
					return;
				}

				const int32 DatabaseTotalFrameNum = DatabaseFrameRanges.FrameRangeSet->GetTotalFrameNum();

				if (DatabaseTotalFrameNum == 0)
				{
					ErrorMessage = LOCTEXT("ControlSetsNoFramesError", "Error: Control Sets contain no frames.");
					return;
				}

				// Compute Frame Attributes

				const int32 AttributeNum = TrainingAutoEncoder->TrainedFrameAttributes.Num();

				TArray<FAnimDatabaseFrameAttribute> TrainingDatabaseFrameAttributes;
				TrainingDatabaseFrameAttributes.SetNum(AttributeNum);
				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					TrainingDatabaseFrameAttributes[AttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(
						TrainingDatabase, 
						TrainingFrameRanges,
						TrainingAutoEncoder->TrainedFrameAttributes[AttributeIdx].FrameAttribute);

					if (!TrainingDatabaseFrameAttributes[AttributeIdx].IsValid() ||
						TrainingAutoEncoder->AttributeTypes[AttributeIdx] != TrainingDatabaseFrameAttributes[AttributeIdx].Type)
					{
						ErrorMessage = LOCTEXT("AttributesTypesNotMatchingError", "Error: Attribute Types don't match previous used for training AutoEncoder.");
						return;
					}
				}

				// Get and check required bones

				const int32 RequiredBoneNum = TrainingAutoEncoder->AutoEncodedRequiredBoneIndices.Num();

				TArray<int32> DatabaseRequiredBones;
				DatabaseRequiredBones.SetNumUninitialized(RequiredBoneNum);
				for (int32 BoneIdx = 0; BoneIdx < RequiredBoneNum; BoneIdx++)
				{
					const FName BoneName = TrainingAutoEncoder->GetBoneName(TrainingAutoEncoder->AutoEncodedRequiredBoneIndices[BoneIdx]);
					DatabaseRequiredBones[BoneIdx] = TrainingDatabase->FindBoneIndex(BoneName);
					if (DatabaseRequiredBones[BoneIdx] == INDEX_NONE)
					{
						ErrorMessage = FText::Format(
							LOCTEXT("MissingBoneError", "Error: Bone {0} from AutoEncoder no longer found in database."),
							FText::FromName(BoneName));
						return;
					}
				}
				
				// Done Checking for Errors

				Controller->AutoEncoder = TrainingAutoEncoder;
				Controller->ControlSchema = ControllerSchema;
				Controller->ControlSchemaElement = SchemaElement;
				Controller->FrameRate = Controller->TrainingSettings->Database->GetFrameRate();

				Controller->TrainedFrameRangesContentHash = UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(TrainingDatabase, TrainingFrameRanges);
				Controller->TrainedAutoEncoderContentHash = Controller->AutoEncoder->GetContentHash();
				Controller->TrainedSchemaCompatibilityHash = Learning::Observation::GetSchemaObjectsCompatibilityHash(
					ControllerSchema.ObservationSchema->Schema, SchemaElement.SchemaElement);

				// Next we allocate a `UE::AnimGen::FPoseData` object. This will contain all of the information for an array of poses including all the bone
				// transforms, root transforms, tags and events. Here we allocate for `DatabaseTotalFrameNum` frames, and then we are going to take slices of this
				// object for each sequence based on what we previous computed in SequenceStarts and SequenceLengths

				UE::AnimDatabase::FPoseData PoseData;
				PoseData.Resize(DatabaseTotalFrameNum, RequiredBoneNum, TrainingAutoEncoder->GetAttributeTypes(), TrainingAutoEncoder->GetAttributeNames());

				// In parallel we are going to sample each animation sequence in the database and write it into slices of the PoseData object.

				TLearningArray<1, int32> DatabaseRangeSequenceIndices;
				DatabaseRangeSequenceIndices.SetNumUninitialized({ DatabaseTotalRangeNum });
				UE::Learning::FrameRangeSet::AllRangeSequences(DatabaseRangeSequenceIndices, *DatabaseFrameRanges.FrameRangeSet);

				// Make sure everything is loaded before looping in parallel

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("LoadingAnimationData", "Loading Animation Data"));
					SlowTask.MakeDialog();

					TrainingDatabase->WaitForCompressionOnAnimSequencesFromArrayView(DatabaseRangeSequenceIndices);
				}

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("SamplingAnimationData", "Sampling Animation Data"));
					SlowTask.MakeDialog();

					ParallelFor(DatabaseTotalRangeNum, [
						this,
						TrainingDatabase,
						&DatabaseRangeSequenceIndices,
						&PoseData,
						&DatabaseFrameRanges,
						&TrainingDatabaseFrameAttributes,
						TrainingAutoEncoder,
						&DatabaseRequiredBones](int32 RangeIdx)
						{
							TrainingDatabase->GetPoseSubsetData(
								PoseData.Slice(
									DatabaseFrameRanges.FrameRangeSet->GetAllRangeOffsets()[RangeIdx],
									DatabaseFrameRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx]),
								DatabaseRangeSequenceIndices[RangeIdx],
								DatabaseFrameRanges.FrameRangeSet->GetAllRangeStarts()[RangeIdx],
								DatabaseRequiredBones,
								TrainingDatabaseFrameAttributes);
						});
				}

				// Extract Pose Vectors

				// Now that we have our pose data, we need to convert it into flat vectors which can go into the auto-encoder and be encoded. So first we allocate
				// a large 2D array PoseVectors of size (DatabaseTotalFrameNum, PoseVectorSize) to fit all of this data, and then in parallel we are going to use 
				// TrainingAutoEncoder->ToPoseVectors to convert from our PoseData into our PoseVectors.

				const int32 PoseVectorSize = TrainingAutoEncoder->GetPoseVectorSize();

				TLearningArray<2, float> PoseVectors;
				PoseVectors.SetNumUninitialized({ DatabaseTotalFrameNum, PoseVectorSize });

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingPoseVectors", "Computing Pose Vectors"));
					SlowTask.MakeDialog();

					UE::Learning::SlicedParallelFor(DatabaseTotalFrameNum, 512, [TrainingAutoEncoder, &PoseVectors, &PoseData](int32 SliceStart, int32 SliceLength)
						{
							TrainingAutoEncoder->ToPoseVectors(
								PoseVectors.Slice(SliceStart, SliceLength),
								PoseData.ConstSlice(SliceStart, SliceLength),
								TrainingAutoEncoder->AutoEncodedRequiredBoneIndices);

							TrainingAutoEncoder->NormalizePoseVectors(PoseVectors.Slice(SliceStart, SliceLength));
						});
				}

				PoseData.Empty();

				// Encode Pose Vectors

				// Now that we have our pose vectors we can encode them using our auto-encoder. First we allocate an output array EncodedVectors of
				// size (DatabaseTotalFrameNum, PoseEncodingSize), then we allocate an Inference object from our auto-encoder network (and specify the batch size 
				// of DatabaseTotalFrameNum). We then evaluate this inference object to encode all of our pose vectors.

				const int32 PoseEncodingSize = TrainingAutoEncoder->GetEncodingSize();

				EncodedVectors = UE::Learning::SharedMemory::Allocate<2, float>({ DatabaseTotalFrameNum, PoseEncodingSize });

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("EncodingPoseVectors", "Encoding Pose Vectors"));
					SlowTask.MakeDialog();

					TSharedPtr<UE::Learning::FNeuralNetworkInference> BatchInference = TrainingAutoEncoder->EncoderNetwork->GetNetwork()->CreateInferenceObject(DatabaseTotalFrameNum);
					BatchInference->Evaluate(EncodedVectors.View, PoseVectors);
					BatchInference.Reset();
				}

				PoseVectors.Empty();

				// Compute and set distribution of the training dataset

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("NormalizeEncodedPoseVectors", "Normalizing Encoded Pose Vectors"));
					SlowTask.MakeDialog();

					Controller->EncodedPoseMeans.SetNumZeroed(PoseEncodingSize);
					Controller->EncodedPoseStds.SetNumZeroed(PoseEncodingSize);
					Controller->EncodedPoseMins.SetNumZeroed(PoseEncodingSize);
					Controller->EncodedPoseMaxs.SetNumZeroed(PoseEncodingSize);

					UE::AnimDatabase::Math::ComputeMeanStd(
						TLearningArrayView<1, float>(Controller->EncodedPoseMeans.GetData(), PoseEncodingSize),
						TLearningArrayView<1, float>(Controller->EncodedPoseStds.GetData(), PoseEncodingSize),
						EncodedVectors.View);

					UE::AnimDatabase::Math::ComputeMinMax(
						TLearningArrayView<1, float>(Controller->EncodedPoseMins.GetData(), PoseEncodingSize),
						TLearningArrayView<1, float>(Controller->EncodedPoseMaxs.GetData(), PoseEncodingSize),
						EncodedVectors.View);

					float EncodedPoseStdStd = 0.0f;
					UE::AnimDatabase::Math::ComputeMeanStd(Controller->EncodedPoseNormalizationScale, EncodedPoseStdStd, Controller->EncodedPoseStds);

					Controller->NormalizeEncodedPoseVectorsInplace(EncodedVectors.View);
				}

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("RecomputePoseVectorsStd", "Recomputing Encoding Pose Vectors Stds"));
					SlowTask.MakeDialog();

					TLearningArray<1, float> NormalizedPoseMeans;
					NormalizedPoseMeans.SetNumZeroed({ PoseEncodingSize });
					Controller->NormalizedPoseStds.SetNumZeroed(PoseEncodingSize);

					UE::AnimDatabase::Math::ComputeMeanStd(
						NormalizedPoseMeans,
						TLearningArrayView<1, float>(Controller->NormalizedPoseStds.GetData(), PoseEncodingSize),
						EncodedVectors.View);

					NormalizedPoseMeans.Empty();
				}

				TLearningArray<2, float> BlockNormalizePoseStds;
				BlockNormalizePoseStds.SetNumUninitialized({ 4, PoseEncodingSize });
				Learning::Array::Copy(BlockNormalizePoseStds[0], TLearningArrayView<1, float>(Controller->NormalizedPoseStds));
				Learning::Array::Copy(BlockNormalizePoseStds[1], TLearningArrayView<1, float>(Controller->NormalizedPoseStds));
				Learning::Array::Copy(BlockNormalizePoseStds[2], TLearningArrayView<1, float>(Controller->NormalizedPoseStds));
				Learning::Array::Copy(BlockNormalizePoseStds[3], TLearningArrayView<1, float>(Controller->NormalizedPoseStds));

				// Evaluate Controls

				// Like with the pose vectors we allocate one large array for the control vectors on each frame of the database.

				const int32 ControlSchemaVectorSize = ControllerSchema.ObservationSchema->Schema.GetObservationVectorSize(SchemaElement.SchemaElement);
				const int32 ControlSchemaEncodedVectorSize = ControllerSchema.ObservationSchema->Schema.GetEncodedVectorSize(SchemaElement.SchemaElement);
				const int32 ControlSchemaDistributionVectorSize = ControllerSchema.ObservationSchema->Schema.GetObservationDistributionVectorSize(SchemaElement.SchemaElement);

				Controller->ControlVectorSize = ControlSchemaVectorSize;
				Controller->EncodedControlVectorSize = ControlSchemaEncodedVectorSize;
				Controller->ControlDistributionVectorSize = ControlSchemaDistributionVectorSize;

				// Here we create a new control object. This is like the pool of control elements we are going to use.

				int32 ControlTotalFrameNum = 0;
				int32 TotalRangeNum = 0;

				for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
				{
					check(ControlSets[ControlSetIdx].GetControls().Num() == ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalFrameNum());
					ControlTotalFrameNum += ControlSets[ControlSetIdx].GetControls().Num();
					TotalRangeNum += ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalRangeNum();
				}

				TArray<int32, TInlineAllocator<32>> RangeSequenceIndices;
				RangeSequenceIndices.SetNumUninitialized(TotalRangeNum);

				RangeControlStarts = UE::Learning::SharedMemory::Allocate<1, int32>({ TotalRangeNum });
				RangeEncodedStarts = UE::Learning::SharedMemory::Allocate<1, int32>({ TotalRangeNum });
				RangeLengths = UE::Learning::SharedMemory::Allocate<1, int32>({ TotalRangeNum });
				ControlVectors = UE::Learning::SharedMemory::Allocate<2, float>({ ControlTotalFrameNum, Controller->ControlVectorSize });
				
				Controller->ControlVectorDistributionMins.SetNumZeroed(ControlSchemaDistributionVectorSize);
				Controller->ControlVectorDistributionMaxs.SetNumZeroed(ControlSchemaDistributionVectorSize);

				UE::Learning::Observation::InitObservationDistributionMinMax(
					Controller->ControlVectorDistributionMins,
					Controller->ControlVectorDistributionMaxs);

				int32 RangeOffset = 0;
				int32 ControlVectorOffset = 0;

				{
					FScopedSlowTask SlowTask(0.0f, LOCTEXT("ComputingControlVectors", "Computing Control Vectors"));
					SlowTask.MakeDialog();

					for (int32 ControlSetIdx = 0; ControlSetIdx < ControlSetNum; ControlSetIdx++)
					{
						const UE::Learning::FFrameRangeSet& ControlSetTag = ControlSets[ControlSetIdx].Data->FrameRangeSet;

						int32 ControlObjectOffset = 0;

						const int32 EntryNum = ControlSetTag.GetEntryNum();

						for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
						{
							const int32 SequenceIdx = ControlSetTag.GetEntrySequence(EntryIdx);
							const int32 RangeNum = ControlSetTag.GetEntryRangeNum(EntryIdx);

							for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
							{
								// Find this range in the database range set

								const int32 TagRangeStartFrame = ControlSetTag.GetEntryRangeStart(EntryIdx, RangeIdx);
								const int32 TagRangeFrameNum = ControlSetTag.GetEntryRangeLength(EntryIdx, RangeIdx);

								int32 DatabaseRangeEntryIdxStart = INDEX_NONE;
								int32 DatabaseRangeRangeIdxStart = INDEX_NONE;
								int32 DatabaseRangeRangeFrameStart = INDEX_NONE;
								int32 DatabaseRangeEntryIdxEnd = INDEX_NONE;
								int32 DatabaseRangeRangeIdxEnd = INDEX_NONE;
								int32 DatabaseRangeRangeFrameEnd = INDEX_NONE;
								const bool bFoundStart = DatabaseFrameRanges.FrameRangeSet->Find(DatabaseRangeEntryIdxStart, DatabaseRangeRangeIdxStart, DatabaseRangeRangeFrameStart, SequenceIdx, TagRangeStartFrame);
								const bool bFoundEnd = DatabaseFrameRanges.FrameRangeSet->Find(DatabaseRangeEntryIdxEnd, DatabaseRangeRangeIdxEnd, DatabaseRangeRangeFrameEnd, SequenceIdx, TagRangeStartFrame + TagRangeFrameNum - 1);

								check(bFoundStart && bFoundEnd && DatabaseRangeEntryIdxStart == DatabaseRangeEntryIdxEnd && DatabaseRangeRangeIdxStart == DatabaseRangeRangeIdxEnd);

								const int32 FrameOffset = DatabaseFrameRanges.FrameRangeSet->GetEntryRangeOffset(DatabaseRangeEntryIdxStart, DatabaseRangeRangeIdxStart);

								RangeSequenceIndices[RangeOffset] = SequenceIdx;
								RangeEncodedStarts.View[RangeOffset] = FrameOffset + DatabaseRangeRangeFrameStart;
								RangeControlStarts.View[RangeOffset] = ControlVectorOffset;
								RangeLengths.View[RangeOffset] = TagRangeFrameNum;

								// Copy Control Vectors

								for (int32 ControlFrameIdx = 0; ControlFrameIdx < TagRangeFrameNum; ControlFrameIdx++)
								{
									UE::Learning::Observation::SetVectorFromObject(
										ControlVectors.View[ControlVectorOffset + ControlFrameIdx],
										ControllerSchema.ObservationSchema->Schema,
										SchemaElement.SchemaElement,
										ControlObject.ObservationObject->Object,
										ControlSets[ControlSetIdx].GetControls()[ControlObjectOffset + ControlFrameIdx].ObjectElement);

									UE::Learning::Observation::FitObservationDistributionMinMax(
										Controller->ControlVectorDistributionMins,
										Controller->ControlVectorDistributionMaxs,
										ControlVectors.View[ControlVectorOffset + ControlFrameIdx],
										ControllerSchema.ObservationSchema->Schema,
										SchemaElement.SchemaElement);
								}

								// Update Offsets

								ControlObjectOffset += TagRangeFrameNum;
								ControlVectorOffset += TagRangeFrameNum;
								RangeOffset++;
							}
						}

						check(ControlObjectOffset == ControlSets[ControlSetIdx].Data->FrameRangeSet.GetTotalFrameNum());
					}
				}

				check(RangeOffset == TotalRangeNum);
				check(ControlVectorOffset == ControlTotalFrameNum);

				ControlObject.ObservationObject->Object.Empty();

				// Make Controller Network

				UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction BuilderActivationFunction = UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
				switch (Controller->TrainingSettings->ActivationFunction)
				{
				case EAnimGenActivationFunction::ReLU: BuilderActivationFunction = UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU; break;
				case EAnimGenActivationFunction::ELU: BuilderActivationFunction = UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ELU; break;
				case EAnimGenActivationFunction::TanH: BuilderActivationFunction = UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::TanH; break;
				case EAnimGenActivationFunction::GELU: BuilderActivationFunction = UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::GELU; break;
				default: checkNoEntry();
				}

				UE::Learning::Observation::FNetworkSettings ControlNetworkSettings;
				ControlNetworkSettings.bUseCompressedLinearLayers = Controller->TrainingSettings->bUseCompressedLinearLayers;
				switch (Controller->TrainingSettings->WeightInit)
				{
				case EAnimGenWeightInit::Gaussian: ControlNetworkSettings.WeightInitialization = UE::Learning::Observation::EWeightInitialization::KaimingGaussian; break;
				case EAnimGenWeightInit::Uniform: ControlNetworkSettings.WeightInitialization = UE::Learning::Observation::EWeightInitialization::KaimingUniform; break;
				default: checkNoEntry();
				}

				UE::NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LODLinearLayerSettings;
				LODLinearLayerSettings.Type = Controller->TrainingSettings->bUseCompressedLinearLayers ? 
					UE::NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
					UE::NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;
				switch (Controller->TrainingSettings->WeightInit)
				{
				case EAnimGenWeightInit::Gaussian: LODLinearLayerSettings.WeightInitializationSettings.Type = UE::NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
				case EAnimGenWeightInit::Uniform: LODLinearLayerSettings.WeightInitializationSettings.Type = UE::NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
				default: checkNoEntry();
				}

				UE::NNE::RuntimeBasic::FModelBuilder ControllerBuilder(UE::Learning::Random::Int(Controller->TrainingSettings->RandomSeed ^ 0x5846ab8d));

				// Here we use the model builder to create the encoder network associated with our schema. This get stored in ControllerEncoderElement so we can
				// embed it into a larger network structure 
				UE::NNE::RuntimeBasic::FModelBuilderElement ControllerEncoderElement;
				UE::Learning::Observation::MakeEncoderNetworkModelBuilderElementFromSchema(
					ControllerEncoderElement,
					ControllerBuilder,
					ControllerSchema.ObservationSchema->Schema,
					SchemaElement.SchemaElement,
					ControlNetworkSettings);

				// Build the rest of the network structure with the Control Encoder embedded
				int32 ControllerCompatibilityHash = 0; // TODO
				TArray<uint8> ControllerFileData;
				uint32 ControllerInputSize, ControllerOutputSize;
				ControllerBuilder.WriteFileDataAndReset(
					ControllerFileData, ControllerInputSize, ControllerOutputSize,
					ControllerEncoderElement);
				Controller->ControlEncoderNetwork = NewObject<ULearningNeuralNetworkData>(Controller);
				Controller->ControlEncoderNetwork->Init(ControllerInputSize, ControllerOutputSize, ControllerCompatibilityHash, ControllerFileData);
				ControllerFileData.Empty();

				ControlEncoderNetworkData = UE::Learning::SharedMemory::Allocate<1, uint8>({ Controller->ControlEncoderNetwork->GetSnapshotByteNum() });
				Controller->ControlEncoderNetwork->SaveToSnapshot(MakeArrayView<uint8>(ControlEncoderNetworkData.View.GetData(), ControlEncoderNetworkData.View.Num()));

				{
					int32 LOD0CompatibilityHash = 0;
					TArray<uint8> LOD0FileData;
					uint32 LOD0InputSize, LOD0OutputSize;
					UE::NNE::RuntimeBasic::FModelBuilder LOD0Builder(UE::Learning::Random::Int(Controller->TrainingSettings->RandomSeed ^ 0x5846ab8c));

					LOD0Builder.WriteFileDataAndReset(
						LOD0FileData, LOD0InputSize, LOD0OutputSize,
						LOD0Builder.MakeSequence({
							LOD0Builder.MakeResidualMLPWithLayerNorm(
								// [ Prev, Noise, Control ]
								PoseEncodingSize * 5 + Controller->EncodedControlVectorSize,
								PoseEncodingSize * 4,
								Controller->TrainingSettings->LOD0HiddenUnitNum,
								Controller->TrainingSettings->LOD0LayerNum,
								BuilderActivationFunction,
								false,
								LODLinearLayerSettings),
							LOD0Builder.MakeDenormalize(PoseEncodingSize * 4,
								LOD0Builder.MakeValuesZero(PoseEncodingSize * 4),
								BlockNormalizePoseStds.Flatten())
							}));

					Controller->LOD0Network = NewObject<ULearningNeuralNetworkData>(Controller);
					Controller->LOD0Network->Init(LOD0InputSize, LOD0OutputSize, LOD0CompatibilityHash, LOD0FileData);
					LOD0FileData.Empty();

					LOD0NetworkData = UE::Learning::SharedMemory::Allocate<1, uint8>({ Controller->LOD0Network->GetSnapshotByteNum() });
					Controller->LOD0Network->SaveToSnapshot(MakeArrayView<uint8>(LOD0NetworkData.View.GetData(), LOD0NetworkData.View.Num()));
				}

				{
					int32 LOD1CompatibilityHash = 0;
					TArray<uint8> LOD1FileData;
					uint32 LOD1InputSize, LOD1OutputSize;
					UE::NNE::RuntimeBasic::FModelBuilder LOD1Builder(UE::Learning::Random::Int(Controller->TrainingSettings->RandomSeed ^ 0x5846ab8c));

					LOD1Builder.WriteFileDataAndReset(
						LOD1FileData, LOD1InputSize, LOD1OutputSize,
						LOD1Builder.MakeSequence({
							LOD1Builder.MakeResidualMLPWithLayerNorm(
								// [ Prev, Noise, Control ]
								PoseEncodingSize * 5 + Controller->EncodedControlVectorSize,
								PoseEncodingSize * 4,
								Controller->TrainingSettings->LOD1HiddenUnitNum,
								Controller->TrainingSettings->LOD1LayerNum,
								BuilderActivationFunction,
								false,
								LODLinearLayerSettings),
							LOD1Builder.MakeDenormalize(PoseEncodingSize * 4,
								LOD1Builder.MakeValuesZero(PoseEncodingSize * 4),
								BlockNormalizePoseStds.Flatten())
							}));

					Controller->LOD1Network = NewObject<ULearningNeuralNetworkData>(Controller);
					Controller->LOD1Network->Init(LOD1InputSize, LOD1OutputSize, LOD1CompatibilityHash, LOD1FileData);
					LOD1FileData.Empty();

					LOD1NetworkData = UE::Learning::SharedMemory::Allocate<1, uint8>({ Controller->LOD1Network->GetSnapshotByteNum() });
					Controller->LOD1Network->SaveToSnapshot(MakeArrayView<uint8>(LOD1NetworkData.View.GetData(), LOD1NetworkData.View.Num()));
				}

				{
					int32 LOD2CompatibilityHash = 0;
					TArray<uint8> LOD2FileData;
					uint32 LOD2InputSize, LOD2OutputSize;
					UE::NNE::RuntimeBasic::FModelBuilder LOD2Builder(UE::Learning::Random::Int(Controller->TrainingSettings->RandomSeed ^ 0x5846ab8c));

					LOD2Builder.WriteFileDataAndReset(
						LOD2FileData, LOD2InputSize, LOD2OutputSize,
						LOD2Builder.MakeSequence({
							LOD2Builder.MakeResidualMLPWithLayerNorm(
								// [ Prev, Noise, Control ]
								PoseEncodingSize * 5 + Controller->EncodedControlVectorSize,
								PoseEncodingSize * 4,
								Controller->TrainingSettings->LOD2HiddenUnitNum,
								Controller->TrainingSettings->LOD2LayerNum,
								BuilderActivationFunction,
								false,
								LODLinearLayerSettings),
							LOD2Builder.MakeDenormalize(PoseEncodingSize * 4,
								LOD2Builder.MakeValuesZero(PoseEncodingSize * 4),
								BlockNormalizePoseStds.Flatten())
							}));

					Controller->LOD2Network = NewObject<ULearningNeuralNetworkData>(Controller);
					Controller->LOD2Network->Init(LOD2InputSize, LOD2OutputSize, LOD2CompatibilityHash, LOD2FileData);
					LOD2FileData.Empty();

					LOD2NetworkData = UE::Learning::SharedMemory::Allocate<1, uint8>({ Controller->LOD2Network->GetSnapshotByteNum() });
					Controller->LOD2Network->SaveToSnapshot(MakeArrayView<uint8>(LOD2NetworkData.View.GetData(), LOD2NetworkData.View.Num()));
				}

				// Set Network Sizes

				Controller->ControlEncoderSize = Controller->ControlEncoderNetwork->GetSnapshotByteNum() / 1000;
				Controller->DenoiserSize = (
					Controller->LOD0Network->GetSnapshotByteNum() +
					Controller->LOD1Network->GetSnapshotByteNum() + 
					Controller->LOD2Network->GetSnapshotByteNum()) / 1000;

                // Update UI

				Controller->TrainingSettings->ForceRefresh();
				ToolkitPtr->TrainingSettingsWidget->ForceRefresh();
				ToolkitPtr->EditingAssetWidget->ForceRefresh();

				// Allocate Control Data

				// We allocate another array of "controls" which we use to synchronize various events between the python training process and the C++ side.

				TrainProcessControls = UE::Learning::SharedMemory::Allocate<1, volatile int32>({ TrainControlNum });
				UE::Learning::Array::Zero(TrainProcessControls.View);

				// Write Config File

				// We pass all of the parameters we need to give to the python process by writing a JSON config file it can load as soon as it starts

				const FString EnginePath = FPaths::EngineDir();
				const FString SitePackagesPath = UE::Learning::Trainer::GetSitePackagesPath(EnginePath);
				const FString IntermediatePath = FPaths::ProjectIntermediateDir() / TEXT("AnimGen");
				const FString TaskName = Controller->GetName();

				IFileManager& FileManager = IFileManager::Get();
				const FString TimeStamp = FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S"));
				const FString ConfigPath = IntermediatePath / TEXT("Configs") / FString::Printf(TEXT("%s_%s.json"), *TaskName, *TimeStamp);

				TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

				ConfigObject->SetStringField(TEXT("TaskName"), *TaskName);
				ConfigObject->SetStringField(TEXT("TimeStamp"), *TimeStamp);

				ConfigObject->SetStringField(TEXT("EnginePath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*EnginePath));
				ConfigObject->SetStringField(TEXT("SitePackagesPath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*SitePackagesPath));
				ConfigObject->SetStringField(TEXT("IntermediatePath"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*IntermediatePath));

				ConfigObject->SetNumberField(TEXT("RangeNum"), TotalRangeNum);
				ConfigObject->SetNumberField(TEXT("DatabaseTotalFrameNum"), DatabaseTotalFrameNum);
				ConfigObject->SetNumberField(TEXT("ControlTotalFrameNum"), ControlTotalFrameNum);

				ConfigObject->SetNumberField(TEXT("ControllerByteNum"), Controller->ControlEncoderNetwork->GetSnapshotByteNum());
				ConfigObject->SetNumberField(TEXT("LOD0ByteNum"), Controller->LOD0Network->GetSnapshotByteNum());
				ConfigObject->SetNumberField(TEXT("LOD1ByteNum"), Controller->LOD1Network->GetSnapshotByteNum());
				ConfigObject->SetNumberField(TEXT("LOD2ByteNum"), Controller->LOD2Network->GetSnapshotByteNum());

				ConfigObject->SetNumberField(TEXT("DeltaTime"), 1.0f / Controller->FrameRate.AsDecimal());
				ConfigObject->SetNumberField(TEXT("ControlVectorSize"), Controller->ControlVectorSize);
				ConfigObject->SetNumberField(TEXT("EncodedControlVectorSize"), Controller->EncodedControlVectorSize);
				ConfigObject->SetNumberField(TEXT("PoseEncodingSize"), PoseEncodingSize);
				ConfigObject->SetNumberField(TEXT("DenoiserSteps"), Controller->TrainingSettings->DenoiserSteps);
				ConfigObject->SetNumberField(TEXT("DenoiserLayerNum"), Controller->TrainingSettings->DenoiserLayerNum);
				ConfigObject->SetNumberField(TEXT("DenoiserHiddenUnitNum"), Controller->TrainingSettings->DenoiserHiddenUnitNum);

				ConfigObject->SetNumberField(TEXT("IterationNum"), Controller->TrainingSettings->NumberOfTrainingIterations);
				ConfigObject->SetNumberField(TEXT("IterationDistillNum"), Controller->TrainingSettings->NumberOfDistillationIterations);
				ConfigObject->SetNumberField(TEXT("WarmupIterations"), Controller->TrainingSettings->WarmupIterations);
				ConfigObject->SetNumberField(TEXT("LearningRate"), Controller->TrainingSettings->LearningRate);
				ConfigObject->SetNumberField(TEXT("BatchSize"), Controller->TrainingSettings->BatchSize);
				ConfigObject->SetNumberField(TEXT("Seed"), Controller->TrainingSettings->RandomSeed);
				ConfigObject->SetStringField(TEXT("Device"), Controller->TrainingSettings->Device == EAnimGenTrainingDevice::GPU ? TEXT("GPU") : TEXT("CPU"));

				ConfigObject->SetNumberField(TEXT("PoseNoiseScale"), Controller->TrainingSettings->PoseNoiseScale);
				ConfigObject->SetNumberField(TEXT("ControlNoiseScale"), Controller->TrainingSettings->ControlNoiseScale);
				ConfigObject->SetNumberField(TEXT("RandomPoseSampleRate"), Controller->TrainingSettings->RandomPoseSampleRate);

				TArray<TSharedPtr<FJsonValue>> JSONRangeSequenceNames;
				JSONRangeSequenceNames.Reserve(TotalRangeNum);
				for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
				{
					if (const UAnimSequence* AnimSequence = TrainingDatabase->GetAnimSequence(RangeSequenceIndices[RangeIdx]))
					{
						JSONRangeSequenceNames.Emplace(MakeShared<FJsonValueString>(AnimSequence->GetName()));
					}
					else
					{
						JSONRangeSequenceNames.Emplace(MakeShared<FJsonValueString>(TEXT("nullptr")));

					}
				}
				ConfigObject->SetArrayField(TEXT("RangeSequenceNames"), JSONRangeSequenceNames);

				TArray<TSharedPtr<FJsonValue>> JSONNormalizedPoseStds;
				JSONNormalizedPoseStds.Reserve(PoseEncodingSize);
				for (int32 EncIdx = 0; EncIdx < PoseEncodingSize; EncIdx++)
				{
					JSONNormalizedPoseStds.Emplace(MakeShared<FJsonValueNumber>(Controller->NormalizedPoseStds[EncIdx]));
				}
				ConfigObject->SetArrayField(TEXT("NormalizedPoseStds"), JSONNormalizedPoseStds);

				ConfigObject->SetObjectField(TEXT("ControlSchema"), UE::Learning::Trainer::ConvertObservationSchemaToJSON(ControllerSchema.ObservationSchema->Schema, SchemaElement.SchemaElement));

				// To let the Python use the arrays we created we pass it the Guids of the shared memory.

				ConfigObject->SetStringField(TEXT("ControlGuid"), *TrainProcessControls.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("ControllerGuid"), *ControlEncoderNetworkData.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("LOD0Guid"), *LOD0NetworkData.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("LOD1Guid"), *LOD1NetworkData.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("LOD2Guid"), *LOD2NetworkData.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("RangeControlStartsGuid"), *RangeControlStarts.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("RangeEncodedStartsGuid"), *RangeEncodedStarts.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("RangeLengthsGuid"), *RangeLengths.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("ControlVectorsGuid"), *ControlVectors.GetPlatformGuidString());
				ConfigObject->SetStringField(TEXT("EncodedVectorsGuid"), *EncodedVectors.GetPlatformGuidString());

				FString JsonString;
				TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
				FJsonSerializer::Serialize(ConfigObject, JsonWriter, true);

				FFileHelper::SaveStringToFile(JsonString, *ConfigPath);

				// Reset Iterations

				MaxTrainingIterations = Controller->TrainingSettings->NumberOfTrainingIterations;
				MaxDistillIterations = Controller->TrainingSettings->NumberOfDistillationIterations;
				TrainingIteration = 0;
				DistillIteration = 0;
				StartTrainingTimeIteration = -1;
				StartDistillTimeIteration = -1;

				// Launch Training Process

				const FString PythonScriptPath = FPaths::EngineDir() / TEXT("Plugins/Experimental/Animation/AnimGen/Content/Python/train_controller.py");
				const FString PythonExecutablePath = UE::Learning::Trainer::GetPythonExecutablePath(FPaths::ProjectIntermediateDir());

				const FString CommandLineArguments = FString::Printf(TEXT("\"%s\" \"%s\""),
					*FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonScriptPath),
					*FileManager.ConvertToAbsolutePathForExternalAppForRead(*ConfigPath));

				UE::Learning::ESubprocessFlags SubprocessFlags = UE::Learning::ESubprocessFlags::None;
				if (!Controller->TrainingSettings->bEnableLogging) { SubprocessFlags |= UE::Learning::ESubprocessFlags::NoRedirectOutput; }

				check(!TrainProcess.IsRunning());
				TrainProcess.Launch(
					FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonExecutablePath),
					CommandLineArguments,
					SubprocessFlags);

				TrainingProgressNotification = FSlateNotificationManager::Get().StartProgressNotification(
					FText::Format(LOCTEXT("ControllerProgressTrainingMessage", "Training {0}"), FText::FromString(Controller->GetName())),
					MaxTrainingIterations);

				DistillProgressNotification = FSlateNotificationManager::Get().StartProgressNotification(
					FText::Format(LOCTEXT("ControllerProgressDistillingMessage", "Distilling {0}"), FText::FromString(Controller->GetName())),
					MaxDistillIterations);

				bTrainingRunning = true;
			}
		}

		virtual void StopTraining() override
		{
			if (!IsTraining()) { return; }

			TrainProcessControls.View[TrainControlExit] = 1;

			float Timer = 0.0f;
			const float SleepDuration = 0.01f;
			const float Timeout = 5.0f;

			while (TrainProcess.Update())
			{
				if (Timer > Timeout)
				{
					TrainProcess.Terminate();
					break;
				}

				Timer += SleepDuration;
				FPlatformProcess::Sleep(SleepDuration);
			}

			DoneTraining();
		}

		void DoneTraining()
		{
			bTrainingRunning = false;
			TrainProcess.Terminate();
			if (TrainingProgressNotification.IsValid()) { FSlateNotificationManager::Get().CancelProgressNotification(TrainingProgressNotification); }
			if (DistillProgressNotification.IsValid()) { FSlateNotificationManager::Get().CancelProgressNotification(DistillProgressNotification); }
			TrainingProgressNotification.Reset();
			DistillProgressNotification.Reset();
			MaxTrainingIterations = 0;
			MaxDistillIterations = 0;
			TrainingIteration = 0;
			DistillIteration = 0;

			Learning::SharedMemory::Deallocate(TrainProcessControls);

			Learning::SharedMemory::Deallocate(ControlEncoderNetworkData);
			Learning::SharedMemory::Deallocate(LOD0NetworkData);
			Learning::SharedMemory::Deallocate(LOD1NetworkData);
			Learning::SharedMemory::Deallocate(LOD2NetworkData);

			Learning::SharedMemory::Deallocate(RangeControlStarts);
			Learning::SharedMemory::Deallocate(RangeEncodedStarts);
			Learning::SharedMemory::Deallocate(RangeLengths);

			Learning::SharedMemory::Deallocate(ControlVectors);
			Learning::SharedMemory::Deallocate(EncodedVectors);
		}

		virtual bool IsTraining() const override { return bTrainingRunning; }

		virtual int32 GetProgressBarNum() const override { return 2; }

		virtual FText GetProcessName(const int32 ProgressBarIdx) const override
		{
			return ProgressBarIdx == 0 ? LOCTEXT("TrainingProcess", "Training") : LOCTEXT("DistillingProcess", "Distilling");
		}

		virtual ETrainingStatus GetTrainingStatus(const int32 ProgressBarIdx) const override
		{
			const int32 CurrentIteration = GetIterationNum(ProgressBarIdx);
			const int32 MaxIteration = GetMaxIterationNum(ProgressBarIdx);

			if ((!IsTraining() && Toolkit.IsValid() && Toolkit.Pin()->Controller->IsValid()) ||
				(IsTraining() && CurrentIteration == MaxIteration - 1))
			{
				return ETrainingStatus::Done;
			}
			else if (IsTraining() && CurrentIteration == 0)
			{
				return ETrainingStatus::Preparing;
			}
			else if (IsTraining() && CurrentIteration > 0)
			{
				return ETrainingStatus::Training;
			}
			else
			{
				return ETrainingStatus::NotStarted;
			}
		}

		virtual FText GetErrorMessage() const override
		{
			return ErrorMessage;
		}

		virtual int32 GetIterationNum(const int32 ProgressBarIdx) const override
		{
			return ProgressBarIdx == 0 ? TrainingIteration : DistillIteration;
		}

		virtual int32 GetMaxIterationNum(const int32 ProgressBarIdx) const override
		{
			return ProgressBarIdx == 0 ? MaxTrainingIterations : MaxDistillIterations;
		}

		virtual float GetTrainingLoss(const int32 ProgressBarIdx) const override
		{
			const int32 ControlIdx = ProgressBarIdx == 0 ? TrainControlTrainingLoss : TrainControlDistillLoss;
			return IsTraining() ? ((float)TrainProcessControls.View[ControlIdx]) / 100000.0f : 0.0f;
		}

		virtual TOptional<FTimespan> GetEstimateTimeRemaining(const int32 ProgressBarIdx) const override
		{
			const FDateTime StartTime = ProgressBarIdx == 0 ? StartTrainingTime : StartDistillTime;
			const int32 StartTimeIteration = ProgressBarIdx == 0 ? StartTrainingTimeIteration : StartDistillTimeIteration;
			const int32 CurrentIteration = GetIterationNum(ProgressBarIdx);
			const int32 MaxIterations = GetMaxIterationNum(ProgressBarIdx);

			if (StartTimeIteration != -1 && CurrentIteration > StartTimeIteration + 250)
			{
				const FTimespan ElapsedTime = FDateTime::Now() - StartTime;
				const int32 ElapsedIteration = CurrentIteration - StartTimeIteration;
				const int32 RemainingIteration = MaxIterations - CurrentIteration;
				const float IterationRatio = (float)RemainingIteration / (float)ElapsedIteration;
				return IterationRatio * ElapsedTime;
			}

			return TOptional<FTimespan>();
		}

		void Update()
		{
			if (TSharedPtr<FControllerToolkit> ToolkitPtr = Toolkit.Pin())
			{
				UAnimGenController* Controller = ToolkitPtr->Controller;

				if (!bTrainingRunning) { return; }

				if (TrainProcess.Update())
				{
					TrainingIteration = TrainProcessControls.View[TrainControlTrainingIteration];
					DistillIteration = TrainProcessControls.View[TrainControlDistillIteration];

					// Update Estimated Time

					if (TrainingIteration > 250 && StartTrainingTimeIteration == -1)
					{
						StartTrainingTimeIteration = TrainingIteration;
						StartTrainingTime = FDateTime::Now();
					}

					if (DistillIteration > 250 && StartDistillTimeIteration == -1)
					{
						StartDistillTimeIteration = DistillIteration;
						StartDistillTime = FDateTime::Now();
					}

					// Update Progress Notification

					if (TrainingProgressNotification.IsValid())
					{
						FSlateNotificationManager::Get().UpdateProgressNotification(TrainingProgressNotification, TrainingIteration + 1);
					}

					if (DistillProgressNotification.IsValid())
					{
						FSlateNotificationManager::Get().UpdateProgressNotification(DistillProgressNotification, DistillIteration + 1);
					}

					// Update Networks

					if (Controller->ControlEncoderNetwork &&
						ControlEncoderNetworkData.View.GetData() &&
						TrainProcessControls.View[TrainControlControlEncoderNetworkUpdate])
					{
						Controller->ControlEncoderNetwork->LoadFromSnapshot(ControlEncoderNetworkData.View);
						Controller->Modify();
						TrainProcessControls.View[TrainControlControlEncoderNetworkUpdate] = 0;
					}

					if (Controller->LOD0Network && Controller->LOD1Network && Controller->LOD2Network &&
						LOD0NetworkData.View.GetData() && LOD1NetworkData.View.GetData() && LOD2NetworkData.View.GetData() &&
						TrainProcessControls.View[TrainControlLODNetworkUpdate])
					{
						Controller->LOD0Network->LoadFromSnapshot(LOD0NetworkData.View);
						Controller->LOD1Network->LoadFromSnapshot(LOD1NetworkData.View);
						Controller->LOD2Network->LoadFromSnapshot(LOD2NetworkData.View);
						Controller->Modify();
						TrainProcessControls.View[TrainControlLODNetworkUpdate] = 0;
					}
				}
				else
				{
					DoneTraining();
				}
			}
		}

		static constexpr int32 TrainControlExit = 0;
		static constexpr int32 TrainControlTrainingIteration = 1;
		static constexpr int32 TrainControlDistillIteration = 2;
		static constexpr int32 TrainControlTrainingLoss = 3;
		static constexpr int32 TrainControlDistillLoss = 4;
		static constexpr int32 TrainControlControlEncoderNetworkUpdate = 5;
		static constexpr int32 TrainControlLODNetworkUpdate = 6;
		static constexpr int32 TrainControlNum = 7;

		TWeakPtr<FControllerToolkit> Toolkit;

		Learning::TSharedMemoryArrayView<1, volatile int32> TrainProcessControls;

		Learning::TSharedMemoryArrayView<1, uint8> ControlEncoderNetworkData;
		Learning::TSharedMemoryArrayView<1, uint8> LOD0NetworkData;
		Learning::TSharedMemoryArrayView<1, uint8> LOD1NetworkData;
		Learning::TSharedMemoryArrayView<1, uint8> LOD2NetworkData;

		Learning::TSharedMemoryArrayView<1, int32> RangeControlStarts;
		Learning::TSharedMemoryArrayView<1, int32> RangeEncodedStarts;
		Learning::TSharedMemoryArrayView<1, int32> RangeLengths;

		Learning::TSharedMemoryArrayView<2, float> ControlVectors;
		Learning::TSharedMemoryArrayView<2, float> EncodedVectors;

		bool bTrainingRunning = false;
		Learning::FSubprocess TrainProcess;
		int32 TrainingIteration = 0;
		int32 DistillIteration = 0;
		int32 MaxTrainingIterations = 0;
		int32 MaxDistillIterations = 0;
		FText ErrorMessage;
		FProgressNotificationHandle TrainingProgressNotification;
		FProgressNotificationHandle DistillProgressNotification;
		int32 StartTrainingTimeIteration = -1;
		int32 StartDistillTimeIteration = -1;
		FDateTime StartTrainingTime;
		FDateTime StartDistillTime;
	};

	static const FName ControllerTabAssetDetails(TEXT("AnimGenEditorControllerAssetDetailsTabID"));
	static const FName ControllerTabViewportSettings(TEXT("AnimGenEditorControllerViewportSettingsTabID"));
	static const FName ControllerTabPreviewSettings(TEXT("AnimGenEditorControllerPreviewSettingsTabID"));
	static const FName ControllerTabViewport(TEXT("AnimGenEditorControllerViewportTabID"));
	static const FName ControllerTabTimeline(TEXT("AnimGenEditorControllerTimelineTabID"));
	static const FName ControllerTabTrainingSettings(TEXT("AnimGenEditorControllerTrainingSettingsTabID"));
	static const FName ControllerTabTraining(TEXT("AnimGenEditorControllerTrainingTabID"));
	static const FName ControllerTabControlObject(TEXT("AnimGenEditorControllerControlObjectTabID"));

	void FControllerToolkit::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimGenController* InController)
	{
		Controller = InController;
		Controller->TrainingSettings->LoadDatabaseAsync();
		Controller->TrainingSettings->LoadAutoEncoderAsync();

		PreviewScene = MakeShared<AnimDatabase::Editor::FPreviewScene>(AnimDatabase::Editor::FPreviewScene::ConstructionValues().ShouldSimulatePhysics(true).ForceUseMovementComponentInNonGameWorld(true));
		//Temporary fix for missing attached assets - MDW (Copied from FPersonaToolkit::CreatePreviewScene)
		PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);

		// Create Timeline Model
		TimelineModel = MakeShared<AnimDatabase::Editor::FTimelineModel>();
		TimelineModel->SetTimelineSnapOnFrames(true); // By default we want to snap on frames when examining data with the controller

		// Create Timeline Tracks Model
		TracksModel = MakeShared<AnimDatabase::Editor::FTimelineTracksModel>();

		// Create Training Model
		TrainingModel = MakeShared<FControllerTrainingModel>(StaticCastWeakPtr<FControllerToolkit>(AsWeak()));

		// Create viewport widget
		ViewportWidget = SNew(SControllerViewport, StaticCastSharedRef<FControllerToolkit>(AsShared()), PreviewScene.ToSharedRef(), FControllerMode::EditorModeId);
		TimelineWidget = SNew(AnimDatabase::Editor::STimeline, TimelineModel.ToWeakPtr(), TracksModel.ToWeakPtr());

		// Create Asset Details widget
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;

		EditingAssetWidget = PropertyModule.CreateDetailView(Args);
		EditingAssetWidget->SetObject(InController);

		ViewportSettingsWidget = PropertyModule.CreateDetailView(Args);
		ViewportSettingsWidget->SetObject(InController->ViewportSettings);

		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		PreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());

		TrainingSettingsWidget = PropertyModule.CreateDetailView(Args);
		TrainingSettingsWidget->SetObject(InController->TrainingSettings);

		TrainingWidget = SNew(STraining, TrainingModel.ToWeakPtr());
		ControlObjectWidget = SNew(SControlObject);

		// Define Editor Layout
		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_AnimGenEditorController_Layout_v0.08")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.8f)
							->AddTab(ControllerTabTrainingSettings, ETabState::OpenedTab)
							->SetForegroundTab(ControllerTabTrainingSettings)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->AddTab(ControllerTabTraining, ETabState::OpenedTab)
							->AddTab(ControllerTabControlObject, ETabState::OpenedTab)
							->SetForegroundTab(ControllerTabTraining)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.6f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.9f)
							->AddTab(ControllerTabViewport, ETabState::OpenedTab)->SetHideTabWell(true)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)
							->AddTab(ControllerTabTimeline, ETabState::OpenedTab)->SetHideTabWell(true)
						)
					)
					->Split(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(ControllerTabAssetDetails, ETabState::OpenedTab)
							->AddTab(ControllerTabPreviewSettings, ETabState::OpenedTab)
							->SetForegroundTab(ControllerTabAssetDetails)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(ControllerTabViewportSettings, ETabState::OpenedTab)
							->SetForegroundTab(ControllerTabViewportSettings)
						)
					)
				)
			);

		const bool bCreateDefaultStandaloneMenuParam = true;
		const bool bCreateDefaultToolbarParam = true;
		const bool bIsToolbarFocusableParam = false;
		FAssetEditorToolkit::InitAssetEditor(
			Mode, InitToolkitHost, 
			TEXT("AnimGenEditorControllerApp"),
			StandaloneDefaultLayout, 
			bCreateDefaultStandaloneMenuParam, 
			bCreateDefaultToolbarParam, 
			InController,
			bIsToolbarFocusableParam);
	}

	void FControllerToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_AnimGenEditorController", "Animation Generation Controller Editor"));
		auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(ControllerTabViewport, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"))[ViewportWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(ControllerTabAssetDetails, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("AssetDetailsTab_Title", "Asset Details"))[EditingAssetWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("AssetDetailsTab", "Controller Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(ControllerTabViewportSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ViewportSettingsTab_Title", "Viewport Settings"))[ViewportSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("ViewportSettingsTab", "Viewport Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(ControllerTabPreviewSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("PreviewSettingsTab_Title", "Preview Settings"))[PreviewSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("PreviewSettingsTab", "Preview Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(ControllerTabTimeline, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TimelineTab_Title", "Timeline"))[TimelineWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TimelineTab", "Timeline"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(ControllerTabTrainingSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TrainingSettingsTab_Title", "Training Settings"))[TrainingSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TrainingSettingsTab", "Training Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(ControllerTabTraining, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("TrainingTab_Title", "Training"))[TrainingWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("TrainingTab", "Training"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));

		InTabManager->RegisterTabSpawner(ControllerTabControlObject, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ControlObjectTab_Title", "Control Object"))[ControlObjectWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("ControlObjectTab", "Control Object"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	}

	void FControllerToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(ControllerTabViewport);
		InTabManager->UnregisterTabSpawner(ControllerTabAssetDetails);
		InTabManager->UnregisterTabSpawner(ControllerTabViewportSettings);
		InTabManager->UnregisterTabSpawner(ControllerTabPreviewSettings);
		InTabManager->UnregisterTabSpawner(ControllerTabTimeline);
		InTabManager->UnregisterTabSpawner(ControllerTabTrainingSettings);
		InTabManager->UnregisterTabSpawner(ControllerTabTraining);
		InTabManager->UnregisterTabSpawner(ControllerTabControlObject);
	}

	FName FControllerToolkit::GetToolkitFName() const
	{
		return FName("AnimGenEditorController");
	}

	FText FControllerToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("AnimGenEditorControllerAppLabel", "AnimGen Controller Editor");
	}

	FText FControllerToolkit::GetToolkitName() const
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::FromString(Controller->GetName()));
		return FText::Format(LOCTEXT("AnimGenEditorControllerToolkitName", "{AssetName}"), Args);
	}

	FLinearColor FControllerToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}

	FString FControllerToolkit::GetWorldCentricTabPrefix() const
	{
		return TEXT("AnimGenEditorController");
	}

	void FControllerToolkit::Tick(float DeltaTime)
	{
		TrainingModel->Update();
	}

	TStatId FControllerToolkit::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimGenEditorToolkit, STATGROUP_Tickables);
	}
}

#undef LOCTEXT_NAMESPACE

