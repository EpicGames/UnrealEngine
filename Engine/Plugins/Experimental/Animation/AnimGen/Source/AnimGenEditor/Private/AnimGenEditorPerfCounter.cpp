// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGenEditorPerfCounter.h"
#include "HAL/PlatformTime.h"

namespace UE::AnimGen::Editor
{
	FPerfCounter::FPerfCounter()
	{
		Reset();
	}

	void FPerfCounter::Reset()
	{
		CycleIdx = 0;
		const int32 HistoryNum = History.Num();
		for (int32 HistoryIdx = 0; HistoryIdx < HistoryNum; HistoryIdx++)
		{
			History[HistoryIdx] = 0;
		}
	}
    
	void FPerfCounter::Begin()
	{
		BeginCycles = FPlatformTime::Cycles64();
	}

	void FPerfCounter::End()
	{
		History[CycleIdx % History.Num()] = FPlatformTime::Cycles64() - BeginCycles;
		CycleIdx++;
	}

	uint64 FPerfCounter::GetAverage() const
	{
		uint64 Sum = 0;

		const int32 HistoryNum = History.Num();
		for (int32 HistoryIdx = 0; HistoryIdx < HistoryNum; HistoryIdx++)
		{
			Sum += History[HistoryIdx];
		}

		return Sum / HistoryNum;
	}
}