// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_AnimGenController.h"
#include "Animation/AnimRootMotionProvider.h"

#define LOCTEXT_NAMESPACE "AnimGraphNode_AnimGenController"


FLinearColor UAnimGraphNode_AnimGenController::GetNodeTitleColor() const
{
	return FLinearColor(0.92f, 0.67f, 0.20f);
}

FText UAnimGraphNode_AnimGenController::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "AnimGen Machine Learning based Controller");
}

FText UAnimGraphNode_AnimGenController::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "AnimGenController");
}

FText UAnimGraphNode_AnimGenController::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|AnimGen");
}

void UAnimGraphNode_AnimGenController::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

#undef LOCTEXT_NAMESPACE
