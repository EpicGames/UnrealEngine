// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimGenEditorControlObject.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorControlObject"

namespace UE::AnimGen::Editor
{
	void SControlObject::Construct(const FArguments& InArgs)
	{
		ChildSlot
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FControlObjectTreeElement>>)
					.TreeItemsSource(&TreeElements)
					.OnGenerateRow(this, &SControlObject::OnGenerateRow)
					.OnGetChildren(this, &SControlObject::OnGetChildren)
					.OnExpansionChanged(this, &SControlObject::OnExpansionChanged)
					.HeaderRow(
						SNew(SHeaderRow)
						+ SHeaderRow::Column("Icon")
						.DefaultLabel(FText::GetEmpty())
						.FillWidth(0.05f)
						+ SHeaderRow::Column("Tag")
						.DefaultLabel(LOCTEXT("ColumnLabelTag", "Tag"))
						.FillWidth(0.45f)
						+ SHeaderRow::Column("Name")
						.DefaultLabel(LOCTEXT("ColumnLabelName", "Name"))
						.FillWidth(0.15f)
						+ SHeaderRow::Column("Value")
						.DefaultLabel(LOCTEXT("ColumnLabelValue", "Value"))
						.FillWidth(0.35f)
					)
			];

		RefreshExpansion(TreeElements);
	}

	TSharedRef<ITableRow> SControlObject::OnGenerateRow(TSharedPtr<FControlObjectTreeElement> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		class SFilterMultiColumnRow : public SMultiColumnTableRow<TSharedPtr<FControlObjectTreeElement>>
		{
		public:
			SLATE_BEGIN_ARGS(SFilterMultiColumnRow) {}
				SLATE_ARGUMENT(TSharedPtr<FControlObjectTreeElement>, Item)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
			{
				Item = InArgs._Item;
				SMultiColumnTableRow<TSharedPtr<FControlObjectTreeElement>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
			}

			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
			{
				if (ColumnName == "Icon")
				{
					// Draw colored circle icon a.k.a "Range Identifier"
					return SNew(SBox)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.MaxDesiredWidth(10.0f)
						.MaxDesiredHeight(10.0f)
						.MaxAspectRatio(1.0f)
						.MinAspectRatio(1.0f)
						[
							SNew(SImage)
								.ColorAndOpacity(MakeAttributeLambda([this]() { return (FSlateColor)Item->Color; }))
								.Image(FCoreStyle::Get().GetBrush("GraphEditor.PinIcon"))
						];
				}
				else if (ColumnName == "Tag")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.AutoWidth()
						[
							SNew(SExpanderArrow, SharedThis(this))
						]
						+ SHorizontalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.AutoWidth()
						.Padding(4.0f, 1.0f)
						[
							SNew(SImage).Image(MakeAttributeLambda([this]() {

								switch (Item->Type)
								{
								case EControlObjectTreeElementType::Null: return FCoreStyle::Get().GetBrush("Icons.Denied");
								case EControlObjectTreeElementType::Continuous: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.VariableIcon");
								case EControlObjectTreeElementType::NamedDiscreteExclusive: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.VariableIcon");
								case EControlObjectTreeElementType::NamedDiscreteInclusive: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.VariableIcon");
								case EControlObjectTreeElementType::And: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.MapValueVariableIcon");
								case EControlObjectTreeElementType::OrExclusive: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.MapValueVariableIcon");
								case EControlObjectTreeElementType::OrInclusive: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.MapValueVariableIcon");
								case EControlObjectTreeElementType::Array: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.ArrayVariableIcon");
								case EControlObjectTreeElementType::Set: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.SetVariableIcon");
								case EControlObjectTreeElementType::Encoding: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.FunctionIcon");
								case EControlObjectTreeElementType::Sparse: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.VariableIcon");
								case EControlObjectTreeElementType::NamedSparse: return FCoreStyle::Get().GetBrush("Kismet.AllClasses.VariableIcon");
								default: return FCoreStyle::Get().GetBrush("Level.EmptyIcon16x");
								}

								}))
						]
						+ SHorizontalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.AutoWidth()
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(MakeAttributeLambda([this]() { return Item->Tag == NAME_None ? FText() : FText::FromName(Item->Tag); }))
						];
				}
				else if (ColumnName == "Name")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(MakeAttributeLambda([this]() { return Item->Name == NAME_None ? FText() : FText::FromName(Item->Name); }))
						];
				}
				else if (ColumnName == "Value")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(MakeAttributeLambda([this]() { 
								
								if (Item->Type == EControlObjectTreeElementType::Continuous)
								{
									const int32 ItemNum = Item->ContinuousValues.Num();

									FString Output = TEXT("[");
									for (int32 Idx = 0; Idx < ItemNum; Idx++)
									{
										Output.Append(FString::Printf(TEXT("% 7.2f"), Item->ContinuousValues[Idx]));
										if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
									}
									Output.Append(TEXT("]"));

									return FText::FromString(Output);
								}
								else if (Item->Type == EControlObjectTreeElementType::DiscreteExclusive)
								{
									return FText::FromString(FString::FromInt(Item->DiscreteValues[0]));
								}
								else if (Item->Type == EControlObjectTreeElementType::DiscreteInclusive)
								{
									const int32 ItemNum = Item->DiscreteValues.Num();

									FString Output = TEXT("[");
									for (int32 Idx = 0; Idx < ItemNum; Idx++)
									{
										Output.Append(FString::FromInt(Item->DiscreteValues[Idx]));
										if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
									}
									Output.Append(TEXT("]"));

									return FText::FromString(Output);
								}
								else if (Item->Type == EControlObjectTreeElementType::NamedDiscreteExclusive)
								{
									return FText::FromName(Item->NamedValues[0]);
								}
								else if (Item->Type == EControlObjectTreeElementType::NamedDiscreteInclusive)
								{
									const int32 ItemNum = Item->NamedValues.Num();

									FString Output = TEXT("[");
									for (int32 Idx = 0; Idx < ItemNum; Idx++)
									{
										Output.Append(Item->NamedValues[Idx].ToString());
										if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
									}
									Output.Append(TEXT("]"));

									return FText::FromString(Output);
								}
								else if (Item->Type == EControlObjectTreeElementType::Sparse)
								{
									const int32 ItemNum = Item->DiscreteValues.Num();

									FString Output = TEXT("[");
									for (int32 Idx = 0; Idx < ItemNum; Idx++)
									{
										Output.Append(FString::Printf(TEXT("%i: % 7.2f"), Item->DiscreteValues[Idx], Item->ContinuousValues[Idx]));
										if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
									}
									Output.Append(TEXT("]"));

									return FText::FromString(Output);
								}
								else if (Item->Type == EControlObjectTreeElementType::NamedSparse)
								{
									const int32 ItemNum = Item->NamedValues.Num();

									FString Output = TEXT("[");
									for (int32 Idx = 0; Idx < ItemNum; Idx++)
									{
										Output.Append(FString::Printf(TEXT("%s: % 7.2f"), *Item->NamedValues[Idx].ToString(), Item->ContinuousValues[Idx]));
										if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
									}
									Output.Append(TEXT("]"));

									return FText::FromString(Output);
								}
								else
								{
									return FText();
								}
							})).Font(FCoreStyle::GetDefaultFontStyle("Mono", 8))
						];
				}

				return SNullWidget::NullWidget;
			}

		private:

			TSharedPtr<FControlObjectTreeElement> Item;
		};

		return SNew(SFilterMultiColumnRow, OwnerTable).Item(Item);
	}

	void SControlObject::OnGetChildren(TSharedPtr<FControlObjectTreeElement> InItem, TArray<TSharedPtr<FControlObjectTreeElement>>& OutChildren)
	{
		OutChildren = InItem->Children;
	}

	void SControlObject::OnExpansionChanged(TSharedPtr<FControlObjectTreeElement> Item, bool bExpanded)
	{
		Item->bIsExpanded = bExpanded;
	}

	void SControlObject::RefreshExpansion(const TArrayView<const TSharedPtr<FControlObjectTreeElement>> Items)
	{
		for (const TSharedPtr<FControlObjectTreeElement>& Item : Items)
		{
			TreeView->SetItemExpansion(Item, Item->bIsExpanded);
			RefreshExpansion(Item->Children);
		}
	}

	void SControlObject::RefreshTreeElements()
	{
		TreeView->RequestTreeRefresh();
		RefreshExpansion(TreeElements);
	}

	TArray<TSharedPtr<FControlObjectTreeElement>>& SControlObject::GetTreeElementsRef()
	{
		return TreeElements;
	}
}

#undef LOCTEXT_NAMESPACE