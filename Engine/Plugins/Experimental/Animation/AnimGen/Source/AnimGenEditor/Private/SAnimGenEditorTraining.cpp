// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimGenEditorTraining.h"
#include "AnimGenEditorTraining.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AnimGenEditorTraining"

namespace UE::AnimGen::Editor
{
	void STraining::Construct(const FArguments& InArgs, const TWeakPtr<ITrainingModel> InTrainingModel)
	{
		TrainingModel = InTrainingModel;

		TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(10.0f, 10.0f)
			.MaxHeight(30.0f)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(10.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
							.Text(LOCTEXT("Train", "Train"))
							.IsEnabled_Lambda([this]() { return TrainingModel.IsValid() ? !TrainingModel.Pin()->IsTraining() : false;  })
							.OnClicked_Lambda([this]() { if (TrainingModel.IsValid()) { TrainingModel.Pin()->StartTraining(); return FReply::Handled(); } return FReply::Unhandled(); })
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(10.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
							.Text(LOCTEXT("Cancel", "Cancel"))
							.IsEnabled_Lambda([this]() { return TrainingModel.IsValid() ? TrainingModel.Pin()->IsTraining() : false;  })
							.OnClicked_Lambda([this]() { if (TrainingModel.IsValid()) { TrainingModel.Pin()->StopTraining(); return FReply::Handled(); } return FReply::Unhandled(); })
					]

			];

		const int32 ProgressBarNum = TrainingModel.IsValid() ? TrainingModel.Pin()->GetProgressBarNum() : 0;

		for (int32 ProgressBarIdx = 0; ProgressBarIdx < ProgressBarNum; ProgressBarIdx++)
		{
			VerticalBox->AddSlot()
				.Padding(10.0f, 10.0f)
				.MaxHeight(20.0f)
				[
					SNew(SProgressBar)
						.Percent_Lambda([this, ProgressBarIdx]()
							{
								if (TrainingModel.IsValid())
								{
									const int32 Iteration = TrainingModel.Pin()->GetIterationNum(ProgressBarIdx);
									const int32 MaxIteration = TrainingModel.Pin()->GetMaxIterationNum(ProgressBarIdx);
									return MaxIteration > 1 ? TOptional<float>((float)Iteration / (MaxIteration - 1)) : TOptional<float>(0.0f);
								}

								return TOptional<float>(0.0f);
							})
				];

			VerticalBox->AddSlot()
				.Padding(10.0f, 10.0f)
				.MaxHeight(30.0f)
				[
					SNew(STextBlock).Text_Lambda([this, ProgressBarIdx]()
						{
							if (TrainingModel.IsValid())
							{
								const TSharedPtr<ITrainingModel> Model = TrainingModel.Pin();

								const FText Name = Model->GetProcessName(ProgressBarIdx);
								const FText ErrorMessage = Model->GetErrorMessage();

								if (!ErrorMessage.IsEmpty()) { return ErrorMessage; }

								switch (Model->GetTrainingStatus(ProgressBarIdx))
								{
								case ETrainingStatus::NotStarted: return FText::Format(LOCTEXT("NotStartedStatus", "Status: {0} not started."), Name);
								case ETrainingStatus::Preparing: return FText::Format(LOCTEXT("PreparingStatus", "Status: {0} waiting..."), Name);
								case ETrainingStatus::Training:
								{
									TOptional<FTimespan> RemainingTime = Model->GetEstimateTimeRemaining(ProgressBarIdx);

									FText TimeRemainingText;
									if (RemainingTime.IsSet())
									{
										FNumberFormattingOptions DigitsFormatOptions;
										DigitsFormatOptions.SetMinimumIntegralDigits(2);
										DigitsFormatOptions.SetMaximumIntegralDigits(2);

										TimeRemainingText = FText::Format(
											LOCTEXT("RemainingTime", ", time remaining {0}:{1}:{2}"),
											FText::AsNumber(RemainingTime->GetDays() * 24 + RemainingTime->GetHours(), &DigitsFormatOptions),
											FText::AsNumber(RemainingTime->GetMinutes(), &DigitsFormatOptions),
											FText::AsNumber(RemainingTime->GetSeconds(), &DigitsFormatOptions));
									}

									FNumberFormattingOptions LossFormatOptions;
									LossFormatOptions.SetMinimumFractionalDigits(5);
									LossFormatOptions.SetMaximumFractionalDigits(5);

									FNumberFormattingOptions IterationFormatOptions;
									IterationFormatOptions.SetMinimumIntegralDigits(7);
									IterationFormatOptions.SetUseGrouping(false);

									return FText::Format(LOCTEXT("TrainingStatus", "Status: {0} iteration {1} of {2}, loss {3}{4}"),
										Name,
										FText::AsNumber(Model->GetIterationNum(ProgressBarIdx), &IterationFormatOptions),
										FText::AsNumber(Model->GetMaxIterationNum(ProgressBarIdx), &IterationFormatOptions),
										FText::AsNumber(Model->GetTrainingLoss(ProgressBarIdx), &LossFormatOptions),
										TimeRemainingText);
								}
								case ETrainingStatus::Done: return FText::Format(LOCTEXT("DoneStatus", "Status: {0} complete."), Name);
								default: return LOCTEXT("ErrorStatus", "Error");
								}
							}
							else
							{
								return FText();
							}
						})
				];
		}


		ChildSlot
			[
				StaticCastSharedRef<SWidget>(VerticalBox)
			];

	}
}

#undef LOCTEXT_NAMESPACE