// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningPCA.h"

#include "AnimGenControl.h"
#include "AnimGenEditorPerfCounter.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabasePose.h"

#include "EdMode.h"

namespace UE::Learning
{
	struct FNeuralNetworkInference;
	struct FPCAEncoder;
}

class ULearningNeuralNetworkData;
struct FAnimDatabaseFrameAttributeEntry;

namespace UE::AnimGen::Editor
{
	class FAutoEncoderToolkit;

	class FAutoEncoderMode : public FEdMode
	{
	public:

		FAutoEncoderMode();
		~FAutoEncoderMode();

		const static FEditorModeID EditorModeId;

		virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);

	private:

		/** Maximum number of characters to draw in the scene */
		const int32 MaxCharacterNum = 128;

		/** Weak pointer to the auto encoder toolkit used for rendering */
		TWeakPtr<FAutoEncoderToolkit> WeakToolkit;

		/** Used to track when the selected ranges have changed and so when the timeline needs updating */
		TArray<int32> ActiveRanges;

		/** Temporary bone parents array */
		TArray<int32> BoneParents;

		/** Character Data */
		TArray<int32> CharacterSequenceIndices;
		TArray<float> CharacterRangeTimes;
		TArray<int32> CharacterRangeStarts;
		TArray<int32> CharacterRangeLengths;

		/** Trained Database content hash */
		int32 CurrentContentHash = 0;

		bool bAutoEncoderValid = false;
		bool bFrameAttributesGenerated = false;
		TArray<FAnimDatabaseFrameAttribute> FrameAttributeObjects;
		TArray<EAnimDatabaseAttributeType> FrameAttributeTypes;
		TArray<FName> FrameAttributeNames;

		TSharedPtr<UE::Learning::FNeuralNetworkInference> EncoderInference;
		TSharedPtr<UE::Learning::FNeuralNetworkInference> DecoderInference;

		ULearningNeuralNetworkData* EncoderInferenceNetwork = nullptr;
		ULearningNeuralNetworkData* DecoderInferenceNetwork = nullptr;

		UE::AnimGen::Editor::FPerfCounter EncoderPerfCounter;
		UE::AnimGen::Editor::FPerfCounter DecoderPerfCounter;

		FAnimDatabasePoseState OriginalPoseState;
		FAnimDatabasePoseState ReconstructedPoseState;
		TLearningArray<1, FVector> ReconstructedRootLocations;
		TLearningArray<1, FQuat4f> ReconstructedRootRotations;

		TLearningArray<2, float> OriginalPoseVector;
		TLearningArray<2, float> EncodedVector;
		TLearningArray<2, float> ReconstructedPoseVector;

		/** Trajectories for all currently queried ranges */
		TArray<TLearningArray<1, FVector>> RangeTrajectoryLocations;
		TArray<TLearningArray<1, FQuat4f>> RangeTrajectoryRotations;

		/** Encoded Space Visualization Data */
		TSharedPtr<Learning::FPCAEncoder> PCAEncoder;
		TLearningArray<2, float> PCAEncodedPoses;
		TLearningArray<1, float> PCAEncoderMean;
		float PCAEncoderStd = 1.0f;
		TLearningArray<2, float> NormalizedEncodedVector;
	};
}