// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "AnimGenControl.h"
#include "AnimGenEditorPerfCounter.h"

#include "AnimDatabase.h"
#include "AnimDatabasePose.h"

#include "EdMode.h"

namespace UE::Learning
{
	struct FNeuralNetworkInference;
}

class ULearningNeuralNetworkData;
class UAnimGenBehavior;
class UAnimGenBehaviorPreview;
struct FAnimDatabaseFrameAttributeEntry;

namespace UE::AnimGen::Editor
{
	class FControllerToolkit;

	class FControllerMode : public FEdMode
	{
	public:

		FControllerMode();
		~FControllerMode();

		const static FEditorModeID EditorModeId;

		virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);

	private:

		/** Maximum number of characters to draw in the scene */
		const int32 MaxCharacterNum = 128;

		int32 CharacterNum = 0;
		TArray<FLinearColor> CharacterColors;

		/** Weak pointer to the controller toolkit used for rendering */
		TWeakPtr<FControllerToolkit> WeakToolkit;

		TArray<int32> ActiveRanges;

		/** Character Data */
		TArray<int32> CharacterSequenceIndices;
		TArray<float> CharacterRangeTimes;
		TArray<int32> CharacterRangeStarts;
		TArray<int32> CharacterRangeLengths;
		TArray<int32> CharacterControlSetIndices;
		TArray<int32> CharacterControlSetEntries;
		TArray<int32> CharacterControlSetRanges;
		TArray<int32> CharacterControlSetFrames;

		/** Trajectories for all currently queried ranges */
		TArray<TLearningArray<1, FVector>> RangeTrajectoryLocations;
		TArray<TLearningArray<1, FQuat4f>> RangeTrajectoryRotations;

		/** Current Frame Attribute Entries */
		TArray<FAnimDatabaseFrameAttributeEntry> FrameAttributeEntries;

		/** Associated Frame Attributes */
		TArray<FAnimDatabaseFrameAttribute> FrameAttributes;
		
		/** Frame Attributes database hash */
		int32 FrameAttributesDatabaseContentHash = 0;

		FAnimDatabasePoseState PoseState;
	};
}
