// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Misc/NotifyHook.h"
#include "Styling/AppStyle.h"
#include "AssetDefinitionDefault.h"
#include "IDetailCustomization.h"
#include "Widgets/Views/SListView.h"
#include "Factories/Factory.h"

#include "AnimGenEditorControllerToolkit.generated.h"

class IDetailsView;
class UAnimGenController;

UCLASS()
class UAnimGenEditorControllerAssetDefinition : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	virtual FText GetAssetDisplayName() const override final;
	virtual FLinearColor GetAssetColor() const override final;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override final;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override final;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override final;
};

/** Asset factory for the AnimGen Controller class */
UCLASS(MinimalAPI, hidecategories = Object)
class UAnimGenControllerEditorFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAnimGenControllerEditorFactory();

	// UFactory overrides.
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetToolTip() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	// ~END UFactory overrides.
};

namespace UE::AnimDatabase::Editor
{
	class FPreviewScene;
	class FViewportClient;
	struct FQueryEntry;
	struct FTimelineModel;
	struct FTimelineTracksModel;
}

namespace UE::AnimGen::Editor
{
	class SControllerViewport;
	struct FControllerTrainingModel;
	class SControlObject;

	/** Details customization used to render the appropriate UI for the UAnimGenControllerTrainingSettings object */
	class FControllerTrainingSettingsDetails : public IDetailCustomization
	{
	public:

		static TSharedRef<IDetailCustomization> MakeInstance();

		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

		/** Callback to generate a single row of the list of ranges */
		TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<AnimDatabase::Editor::FQueryEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);

		/** Callback for when the selection changes */
		void OnSelectionChanged(TSharedPtr<AnimDatabase::Editor::FQueryEntry> SelectedItem, ESelectInfo::Type SelectInfo);

	private:

		/** Keep track of the selected items so we can see when they change */
		TArray<TSharedPtr<AnimDatabase::Editor::FQueryEntry>> SelectedItems;

		/** ListView for all the FQueryEntry items in the query */
		TSharedPtr<SListView<TSharedPtr<AnimDatabase::Editor::FQueryEntry>>> ListView;
	};

	/** Details customization for the controller */
	class FControllerDetails : public IDetailCustomization
	{
	public:

		static TSharedRef<IDetailCustomization> MakeInstance();

		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	};

	class FControllerToolkit : public FAssetEditorToolkit, public FTickableEditorObject
	{
	public:

		void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimGenController* InController);

		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitName() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		void AddToolbarButton(FToolBarBuilder& Builder);

		/** FTickableEditorObject interface */
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		/** End FTickableEditorObject interface */

		/** We keep a pointer to the controller asset. We assume this will out-live the editor toolkit. */
		UAnimGenController* Controller = nullptr;

		/** Model for the timeline Widget */
		TSharedPtr<AnimDatabase::Editor::FTimelineModel> TimelineModel;

		/** Model for the tracks Widget */
		TSharedPtr<AnimDatabase::Editor::FTimelineTracksModel> TracksModel;

		/** Controller Training model */
		TSharedPtr<FControllerTrainingModel> TrainingModel;

		/** Preview Scene for the viewport Widget */
		TSharedPtr<AnimDatabase::Editor::FPreviewScene> PreviewScene;

		/** Viewport Tab Widget */
		TSharedPtr<SControllerViewport> ViewportWidget;

		/** Asset details Tab Widget */
		TSharedPtr<IDetailsView> EditingAssetWidget;

		/** Preview Settings Tab Widget */
		TSharedPtr<SWidget> PreviewSettingsWidget;

		/** Viewport Settings Tab Widget */
		TSharedPtr<IDetailsView> ViewportSettingsWidget;

		/* Timeline Tab Widget */
		TSharedPtr<SWidget> TimelineWidget;

		/** Training Settings Tab Widget */
		TSharedPtr<IDetailsView> TrainingSettingsWidget;

		/* Training Widget */
		TSharedPtr<SWidget> TrainingWidget;

		/* Control Object Widget */
		TSharedPtr<SControlObject> ControlObjectWidget;
	};

}
