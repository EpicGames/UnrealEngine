// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StaticArray.h"

#define UE_API ANIMGENEDITOR_API

namespace UE::AnimGen::Editor
{
    /** Simple performance counter, adapted from FMLDeformerPerfCounter */
	struct FPerfCounter
	{
		UE_API FPerfCounter();

		UE_API void Reset();
		UE_API void Begin();
		UE_API void End();
		UE_API uint64 GetAverage() const;

	private:
    
		uint64 BeginCycles = 0;
        uint64 CycleIdx = 0;
		TStaticArray<uint64, 64> History;
	};
}

#undef UE_API