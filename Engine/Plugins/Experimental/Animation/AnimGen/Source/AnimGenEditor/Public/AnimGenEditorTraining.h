// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"

namespace UE::AnimGen::Editor
{
	enum class ETrainingStatus : uint8
	{
		NotStarted = 0,
		Preparing = 1,
		Training = 2,
		Done = 3,
	};

	struct ITrainingModel
	{
		virtual bool IsEmpty() const { return true; }

		virtual void StartTraining() {}
		virtual void StopTraining() {}
		virtual bool IsTraining() const { return false; }

		virtual int32 GetProgressBarNum() const { return 0; }
		virtual FText GetProcessName(const int32 ProgressBarIdx) const { return NSLOCTEXT("AnimGenTraining", "TrainingProcess", "Training"); }
		virtual int32 GetIterationNum(const int32 ProgressBarIdx) const { return 0; }
		virtual int32 GetMaxIterationNum(const int32 ProgressBarIdx) const { return 0; }
		virtual ETrainingStatus GetTrainingStatus(const int32 ProgressBarIdx) const { return ETrainingStatus::NotStarted; }
		virtual float GetTrainingLoss(const int32 ProgressBarIdx) const { return 0.0f; }
		virtual TOptional<FTimespan> GetEstimateTimeRemaining(const int32 ProgressBarIdx) const { return TOptional<FTimespan>(); };

		virtual FText GetErrorMessage() const { return FText(); }
	};
}