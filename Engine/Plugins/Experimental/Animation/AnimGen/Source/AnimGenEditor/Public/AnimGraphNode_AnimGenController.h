// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNode_AnimGenController.h"
#include "AnimGraphNode_AnimGenController.generated.h"


UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_AnimGenController : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_AnimGenController Node;

	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;

	// UAnimGraphNode_Base interface
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface
};
