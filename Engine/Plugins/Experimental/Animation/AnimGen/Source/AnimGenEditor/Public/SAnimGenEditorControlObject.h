// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

#define UE_API ANIMGENEDITOR_API

namespace UE::AnimGen::Editor
{
	enum class EControlObjectTreeElementType : uint8
	{
		Invalid = 0,
		Null = 1,
		Continuous = 2,
		DiscreteExclusive = 3,
		DiscreteInclusive = 4,
		NamedDiscreteExclusive = 5,
		NamedDiscreteInclusive = 6,
		And = 7,
		OrExclusive = 8,
		OrInclusive = 9,
		Array = 10,
		Set = 11,
		Encoding = 12,
		Sparse = 13,
		NamedSparse = 14,
	};

	struct FControlObjectTreeElement
	{
		EControlObjectTreeElementType Type = EControlObjectTreeElementType::Invalid;
		FName Tag = NAME_None;
		FName Name = NAME_None;
		TArray<TSharedPtr<FControlObjectTreeElement>> Children;
		TArray<int32> DiscreteValues;
		TArray<float> ContinuousValues;
		TArray<FName> NamedValues;
		bool bIsExpanded = true;
		FLinearColor Color = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
	};

	class SControlObject : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SControlObject) {}
		SLATE_END_ARGS();

		UE_API void Construct(const FArguments& InArgs);

		UE_API TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FControlObjectTreeElement> Item, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API void OnGetChildren(TSharedPtr<FControlObjectTreeElement> InItem, TArray<TSharedPtr<FControlObjectTreeElement>>& OutChildren);
		UE_API void OnExpansionChanged(TSharedPtr<FControlObjectTreeElement> Item, bool bExpanded);

		UE_API void RefreshExpansion(const TArrayView<const TSharedPtr<FControlObjectTreeElement>> Items);
		UE_API void RefreshTreeElements();

		UE_API TArray<TSharedPtr<FControlObjectTreeElement>>& GetTreeElementsRef();

	private:

		TArray<TSharedPtr<FControlObjectTreeElement>> TreeElements;

		TSharedPtr<STreeView<TSharedPtr<FControlObjectTreeElement>>> TreeView;
	};
}

#undef UE_API