// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Misc/DateTime.h"

#define UE_API ANIMGENEDITOR_API

namespace UE::AnimGen::Editor
{
    struct ITrainingModel;
    
	class STraining : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(STraining) {}
		SLATE_END_ARGS();

		UE_API void Construct(const FArguments& InArgs, const TWeakPtr<ITrainingModel> InTrainingModel);

	private:

        TWeakPtr<ITrainingModel> TrainingModel;
	};
}

#undef UE_API